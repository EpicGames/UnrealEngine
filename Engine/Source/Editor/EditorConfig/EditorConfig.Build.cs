// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
	public class EditorConfig : ModuleRules
	{
		public EditorConfig(ReadOnlyTargetRules Target) : base(Target)
		{
			// Enable truncation warnings in this module
			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"EditorSubsystem",
					"Engine",
					"Json",
					"UnrealEd",
				} );
		}
	}
}
