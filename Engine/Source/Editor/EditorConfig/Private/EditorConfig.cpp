// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorConfig.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Text.h"
#include "Misc/CoreMiscDefines.h"
#include "Serialization/JsonTypes.h"
#include "UObject/Class.h"
#include "UObject/EnumProperty.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/StrProperty.h"
#include "UObject/TextProperty.h"
#include "UObject/UnrealType.h"

FEditorConfig::FEditorConfig()
{
	JsonConfig = MakeShared<UE::FJsonConfig>();
}

void FEditorConfig::SetParent(TSharedPtr<FEditorConfig> InConfig)
{
	ParentConfig = InConfig;

	if (ParentConfig.IsValid())
	{
		JsonConfig->SetParent(ParentConfig->JsonConfig);
	}
	else
	{
		JsonConfig->SetParent(TSharedPtr<UE::FJsonConfig>());
	}
}

bool FEditorConfig::LoadFromFile(FStringView FilePath)
{
	JsonConfig = MakeShared<UE::FJsonConfig>();
	if (!JsonConfig->LoadFromFile(FilePath))
	{
		return false;
	}

	if (ParentConfig.IsValid())
	{
		JsonConfig->SetParent(ParentConfig->JsonConfig);
	}

	return true;
}
	
bool FEditorConfig::LoadFromString(FStringView Content)
{
	JsonConfig = MakeShared<UE::FJsonConfig>();
	if (!JsonConfig->LoadFromString(Content))
	{
		return false;
	}

	if (ParentConfig.IsValid())
	{
		JsonConfig->SetParent(ParentConfig->JsonConfig);
	}

	return true;
}

bool FEditorConfig::SaveToString(FString& OutResult) const
{
	if (!IsValid())
	{
		return false;
	}

	return JsonConfig->SaveToString(OutResult);
}

bool FEditorConfig::SaveToFile(FStringView FilePath) const
{
	if (!IsValid())
	{
		return false;
	}

	return JsonConfig->SaveToFile(FilePath);
}

bool FEditorConfig::HasOverride(FStringView Key) const
{
	return JsonConfig->HasOverride(UE::FJsonPath(Key));
}

bool FEditorConfig::TryGetRootUObject(const UClass* Class, UObject* OutValue, EPropertyFilter Filter) const
{
	if (!IsValid())
	{
		return false;
	}

	TSharedPtr<FJsonObject> UObjectData = JsonConfig->GetRootObject();
	ReadUObject(UObjectData, Class, OutValue, Filter);

	return true;
}

bool FEditorConfig::TryGetRootStruct(const UStruct* Struct, void* OutValue, EPropertyFilter Filter) const
{
	if (!IsValid())
	{
		return false;
	}

	TSharedPtr<FJsonObject> StructData = JsonConfig->GetRootObject();
	ReadStruct(StructData, Struct, OutValue, nullptr, Filter);

	return true;
}

void FEditorConfig::SetRootUObject(const UClass* Class, const UObject* Instance, EPropertyFilter Filter)
{
	if (!IsValid())
	{
		return;
	}
	
	TSharedPtr<FJsonObject> JsonObject = WriteUObject(Class, Instance, Filter);
	JsonConfig->SetRootObject(JsonObject);
		
	SetDirty();
}

void FEditorConfig::SetRootStruct(const UStruct* Struct, const void* Instance, EPropertyFilter Filter)
{
	if (!IsValid())
	{
		return;
	}
	
	TSharedPtr<FJsonObject> JsonObject = WriteStruct(Struct, Instance, nullptr, Filter);
	JsonConfig->SetRootObject(JsonObject);
		
	SetDirty();
}

void FEditorConfig::ReadStruct(const TSharedPtr<FJsonObject>& JsonObject, const UStruct* Struct, void* Instance, UObject* Owner, EPropertyFilter Filter)
{
	FString TypeName;
	JsonObject->TryGetStringField(TEXT("$type"), TypeName);

	if (!TypeName.IsEmpty() && !ensureAlwaysMsgf(Struct->GetName().Equals(TypeName), TEXT("Type name mismatch in FEditorConfig::ReadUObject. Expected: %s, Actual: %s"), *Struct->GetName(), *TypeName))
	{
		return;
	}

	for (TFieldIterator<FProperty> It(Struct); It; ++It)
	{
		const FProperty* Property = *It;

		if (Filter == EPropertyFilter::MetadataOnly && !Property->HasMetaData("EditorConfig"))
		{
			continue;
		}

		void* DataPtr = Property->ContainerPtrToValuePtr<void>(Instance);
			
		TSharedPtr<FJsonValue> Value = JsonObject->TryGetField(Property->GetName());
		if (Value.IsValid())
		{
			ReadValue(Value, Property, DataPtr, Owner);
		}
	}
}

void FEditorConfig::ReadUObject(const TSharedPtr<FJsonObject>& JsonObject, const UClass* Class, UObject* Instance, EPropertyFilter Filter)
{
	FString TypeName;
	JsonObject->TryGetStringField(TEXT("$type"), TypeName);

	if (!TypeName.IsEmpty() && !ensureAlwaysMsgf(Class->GetName().Equals(TypeName), TEXT("Type name mismatch in FEditorConfig::ReadUObject. Expected: %s, Actual: %s"), *Class->GetName(), *TypeName))
	{
		return;
	}

	for (TFieldIterator<FProperty> It(Class); It; ++It)
	{
		const FProperty* Property = *It;
		
		if (Filter == EPropertyFilter::MetadataOnly && !Property->HasMetaData("EditorConfig"))
		{
			continue;
		}

		void* DataPtr = Property->ContainerPtrToValuePtr<void>(Instance);
		
		TSharedPtr<FJsonValue> Value = JsonObject->TryGetField(Property->GetName());
		if (Value.IsValid())
		{
			ReadValue(Value, Property, DataPtr, Instance);
		}
	}
}

void FEditorConfig::ReadValue(const TSharedPtr<FJsonValue>& JsonValue, const FProperty* Property, void* DataPtr, UObject* Owner)
{
	if (const FStrProperty* StrProperty = CastField<FStrProperty>(Property))
	{
		FString* Value = (FString*) DataPtr;
		JsonValue->TryGetString(*Value);
		return;
	}
	else if (const FNameProperty* NameProperty = CastField<FNameProperty>(Property))
	{
		FString TempValue;
		JsonValue->TryGetString(TempValue);

		*(FName*) DataPtr = *TempValue;
		return;
	}
	else if (const FTextProperty* TextProperty = CastField<FTextProperty>(Property))
	{
		FString TempValue;
		JsonValue->TryGetString(TempValue);

		*(FText*) DataPtr = FText::FromString(TempValue);
		return;
	}
	else if (const FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
	{
		bool Value = BoolProperty->GetDefaultPropertyValue();
		if (JsonValue->TryGetBool(Value))
		{
			BoolProperty->SetPropertyValue(DataPtr, Value);
		}
		return;
	}
	else if (const FFloatProperty* FloatProperty = CastField<FFloatProperty>(Property))
	{
		float* Value = (float*) DataPtr;
		JsonValue->TryGetNumber(*Value);
		return;
	}
	else if (const FDoubleProperty* DoubleProperty = CastField<FDoubleProperty>(Property))
	{
		double* Value = (double*) DataPtr;
		JsonValue->TryGetNumber(*Value);
		return;
	}
	else if (const FInt8Property* Int8Property = CastField<FInt8Property>(Property))
	{
		int8* Value = (int8*) DataPtr;
		JsonValue->TryGetNumber(*Value);
		return;
	}
	else if (const FInt16Property* Int16Property = CastField<FInt16Property>(Property))
	{
		int16* Value = (int16*) DataPtr;
		JsonValue->TryGetNumber(*Value);
		return;
	}
	else if (const FIntProperty* Int32Property = CastField<FIntProperty>(Property))
	{
		int32* Value = (int32*) DataPtr;
		JsonValue->TryGetNumber(*Value);
		return;
	}
	else if (const FInt64Property* Int64Property = CastField<FInt64Property>(Property))
	{
		int64* Value = (int64*) DataPtr;
		JsonValue->TryGetNumber(*Value);
		return;
	}
	else if (const FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
	{
		uint8* Value = (uint8*) DataPtr;
		JsonValue->TryGetNumber(*Value);
		return;
	}
	else if (const FUInt16Property* Uint16Property = CastField<FUInt16Property>(Property))
	{
		uint16* Value = (uint16*) DataPtr;
		JsonValue->TryGetNumber(*Value);
		return;
	}
	else if (const FUInt32Property* Uint32Property = CastField<FUInt32Property>(Property))
	{
		uint32* Value = (uint32*) DataPtr;
		JsonValue->TryGetNumber(*Value);
		return;
	}
	else if (const FUInt64Property* Uint64Property = CastField<FUInt64Property>(Property))
	{
		uint64* Value = (uint64*) DataPtr;
		JsonValue->TryGetNumber(*Value);
		return;
	}
	else if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
	{
		UEnum* Enum = EnumProperty->GetEnum();
		FString ValueString;
		if (JsonValue->TryGetString(ValueString))
		{
			int32 Index = Enum->GetIndexByNameString(ValueString);
			if (Index != INDEX_NONE)
			{
				int64 Value = Enum->GetValueByIndex(Index);
				EnumProperty->GetUnderlyingProperty()->SetIntPropertyValue(DataPtr, Value);
			}
		}
		return;
	}
	else if (const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(Property))
	{
		FString PathString;
		if (JsonValue->TryGetString(PathString))
		{
			Property->ImportText_Direct(*PathString, DataPtr, Owner, 0);
		}
		return;
	}
	else if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
	{
		const TSharedPtr<FJsonObject>* ObjectJsonValue;
		if (JsonValue->TryGetObject(ObjectJsonValue))
		{
			ReadStruct(*ObjectJsonValue, StructProperty->Struct, DataPtr, Owner, EPropertyFilter::All);
		}
		else if (JsonValue->Type == EJson::String)
		{
			FString ImportTextString;
			if (JsonValue->TryGetString(ImportTextString))
			{
				Property->ImportText_Direct(*ImportTextString, DataPtr, Owner, 0);
			}
		}
		return;
	}
	else if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
	{
		const TArray<TSharedPtr<FJsonValue>>* ArrayJsonValue;
		if (JsonValue->TryGetArray(ArrayJsonValue))
		{
			FProperty* InnerProperty = ArrayProperty->Inner;
			FScriptArrayHelper ArrayHelper(ArrayProperty, DataPtr);

			ArrayHelper.EmptyAndAddValues(ArrayJsonValue->Num());

			for (int32 Idx = 0; Idx < ArrayHelper.Num(); ++Idx)
			{
				TSharedPtr<FJsonValue> Value = (*ArrayJsonValue)[Idx];
				ReadValue(Value, InnerProperty, ArrayHelper.GetRawPtr(Idx), Owner);
			}
		}
		return;
	}
	else if (const FSetProperty* SetProperty = CastField<FSetProperty>(Property))
	{
		const TArray<TSharedPtr<FJsonValue>>* SetJsonValue;
		if (JsonValue->TryGetArray(SetJsonValue))
		{
			const FProperty* InnerProperty = SetProperty->ElementProp;
			FScriptSetHelper SetHelper(SetProperty, DataPtr);
			SetHelper.EmptyElements(SetJsonValue->Num());

			// temporary buffer to read elements into
			TArray<uint8> TempBuffer;
			TempBuffer.AddUninitialized(InnerProperty->GetElementSize());

			for (int32 Idx = 0; Idx < SetJsonValue->Num(); ++Idx)
			{
				InnerProperty->InitializeValue(TempBuffer.GetData());

				TSharedPtr<FJsonValue> Value = (*SetJsonValue)[Idx];
				ReadValue(Value, InnerProperty, TempBuffer.GetData(), Owner);

				SetHelper.AddElement(TempBuffer.GetData());

				InnerProperty->DestroyValue(TempBuffer.GetData());
			}
		}
		return;
	}
	else if (const FMapProperty* MapProperty = CastField<FMapProperty>(Property))
	{
		const FProperty* KeyProperty = MapProperty->KeyProp;
		const FProperty* ValueProperty = MapProperty->ValueProp;

		FScriptMapHelper MapHelper(MapProperty, DataPtr);

		// maps can either be stored as a simple JSON object, or as an array of { $key, $value } pairs
		// first check for object storage - this will cover eg. numbers and strings as keys
		const TSharedPtr<FJsonObject>* JsonObjectValue;
		if (JsonValue->TryGetObject(JsonObjectValue))
		{
			MapHelper.EmptyValues((*JsonObjectValue)->Values.Num());

			// temporary buffers to read elements into
			TArray<uint8> TempKey;
			TempKey.AddZeroed(KeyProperty->GetElementSize());

			TArray<uint8> TempValue;
			TempValue.AddZeroed(ValueProperty->GetElementSize());

			for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& JsonPair : (*JsonObjectValue)->Values)
			{
				KeyProperty->InitializeValue(TempKey.GetData());
				KeyProperty->ImportText_Direct(*JsonPair.Key, TempKey.GetData(), Owner, 0);

				ValueProperty->InitializeValue(TempValue.GetData());
				ReadValue(JsonPair.Value, ValueProperty, TempValue.GetData(), Owner);

				MapHelper.AddPair(TempKey.GetData(), TempValue.GetData());

				KeyProperty->DestroyValue(TempKey.GetData());
				ValueProperty->DestroyValue(TempValue.GetData());
			}

			return;
		}

		// then check for array storage, this will cover complex keys eg. custom structs
		const TArray<TSharedPtr<FJsonValue>>* JsonArrayPtr = nullptr;
		if (JsonValue->TryGetArray(JsonArrayPtr))
		{
			MapHelper.EmptyValues(JsonArrayPtr->Num());

			// temporary buffers to read elements into
			TArray<uint8> TempKey;
			TempKey.AddUninitialized(KeyProperty->GetElementSize());

			TArray<uint8> TempValue;
			TempValue.AddUninitialized(ValueProperty->GetElementSize());

			for (const TSharedPtr<FJsonValue>& JsonElement : *JsonArrayPtr)
			{
				TSharedPtr<FJsonObject>* JsonObject = nullptr;
				if (JsonElement->TryGetObject(JsonObject))
				{
					TSharedPtr<FJsonValue> JsonKeyField = (*JsonObject)->TryGetField(TEXT("$key"));
					TSharedPtr<FJsonValue> JsonValueField = (*JsonObject)->TryGetField(TEXT("$value"));

					if (JsonKeyField.IsValid() && JsonValueField.IsValid())
					{
						KeyProperty->InitializeValue(TempKey.GetData());
						ReadValue(JsonKeyField, KeyProperty, TempKey.GetData(), Owner);

						ValueProperty->InitializeValue(TempValue.GetData());
						ReadValue(JsonValueField, ValueProperty, TempValue.GetData(), Owner);

						MapHelper.AddPair(TempKey.GetData(), TempValue.GetData());

						KeyProperty->DestroyValue(TempKey.GetData());
						ValueProperty->DestroyValue(TempValue.GetData());
					}
				}
			}

			return;
		}
	}

	ensureAlwaysMsgf(false, TEXT("Property type is unsupported: %s, type: %s"), *Property->GetPathName(), *Property->GetClass()->GetName());
}

TSharedPtr<FJsonValue> FEditorConfig::WriteArray(const FArrayProperty* ArrayProperty, const void* DataPtr)
{
	FProperty* InnerProperty = ArrayProperty->Inner;
	FScriptArrayHelper ArrayHelper(ArrayProperty, DataPtr);

	TArray<TSharedPtr<FJsonValue>> JsonValuesArray;
	JsonValuesArray.Reserve(ArrayHelper.Num());

	for (int32 Idx = 0; Idx < ArrayHelper.Num(); ++Idx)
	{
		if (ArrayHelper.IsValidIndex(Idx))
		{
			TSharedPtr<FJsonValue> ElementValue = WriteValue(InnerProperty, ArrayHelper.GetRawPtr(Idx), nullptr);
			check(ElementValue.IsValid());
			JsonValuesArray.Add(ElementValue);
		}
	}

	return MakeShared<FJsonValueArray>(JsonValuesArray);
}

TSharedPtr<FJsonValue> FEditorConfig::WriteSet(const FSetProperty* SetProperty, const void* DataPtr)
{
	FProperty* InnerProperty = SetProperty->ElementProp;
	FScriptSetHelper SetHelper(SetProperty, DataPtr);

	TArray<TSharedPtr<FJsonValue>> JsonValuesArray;
	JsonValuesArray.Reserve(SetHelper.Num());

	for (FScriptSetHelper::FIterator It(SetHelper); It; ++It)
	{
		TSharedPtr<FJsonValue> ElementValue = WriteValue(InnerProperty, SetHelper.GetElementPtr(It), nullptr);
		check(ElementValue.IsValid());
		JsonValuesArray.Add(ElementValue);
	}

	return MakeShared<FJsonValueArray>(JsonValuesArray);
}

TSharedPtr<FJsonValue> FEditorConfig::WriteMap(const FMapProperty* MapProperty, const void* DataPtr)
{
	TSharedPtr<FJsonValue> ResultValue;

	FProperty* KeyProperty = MapProperty->KeyProp;
	FProperty* ValueProperty = MapProperty->ValueProp;

	FScriptMapHelper MapHelper(MapProperty, DataPtr);

	if (MapHelper.Num() == 0)
	{
		ResultValue = MakeShared<FJsonValueObject>(MakeShared<FJsonObject>());
	}
	else
	{
		TArray<TSharedPtr<FJsonValue>> JsonKeysArray;
		JsonKeysArray.Reserve(MapHelper.Num());

		TArray<TSharedPtr<FJsonValue>> JsonValuesArray;
		JsonValuesArray.Reserve(MapHelper.Num());

		for (FScriptMapHelper::FIterator It(MapHelper); It; ++It)
		{
			TSharedPtr<FJsonValue> JsonKey = WriteValue(KeyProperty, MapHelper.GetKeyPtr(It), nullptr);
			check(JsonKey.IsValid());
			JsonKeysArray.Add(JsonKey);

			TSharedPtr<FJsonValue> JsonValue = WriteValue(ValueProperty, MapHelper.GetValuePtr(It), nullptr);
			check(JsonValue.IsValid());
			JsonValuesArray.Add(JsonValue);
		}

		// maps can either be stored as $key, $value pairs or, if the keys can be stringified, as a JSON object
		// check Filter we should use based on the first element
		EJson KeyType = JsonKeysArray[0]->Type;
		if (KeyType == EJson::Object)
		{
			TArray<TSharedPtr<FJsonValue>> ResultArray;
			ResultArray.Reserve(MapHelper.Num());

			for (int32 Idx = 0; Idx < MapHelper.Num(); ++Idx)
			{
				if (MapHelper.IsValidIndex(Idx))
				{
					TSharedPtr<FJsonObject> ElementObject = MakeShared<FJsonObject>();
					ElementObject->SetField(TEXT("$key"), JsonKeysArray[Idx]);
					ElementObject->SetField(TEXT("$value"), JsonValuesArray[Idx]);

					ResultArray.Add(MakeShared<FJsonValueObject>(ElementObject));
				}
			}

			ResultValue = MakeShared<FJsonValueArray>(ResultArray);
		}
		else if (KeyType == EJson::Boolean ||
			KeyType == EJson::Number ||
			KeyType == EJson::String)
		{
			TSharedPtr<FJsonObject> ResultObject = MakeShared<FJsonObject>();

			for (int32 Idx = 0; Idx < MapHelper.Num(); ++Idx)
			{
				if (MapHelper.IsValidIndex(Idx))
				{
					FString KeyString;
					const bool bGetStringSuccess = JsonKeysArray[Idx]->TryGetString(KeyString);
					check(bGetStringSuccess);

					ResultObject->SetField(KeyString, JsonValuesArray[Idx]);
				}
			}

			ResultValue = MakeShared<FJsonValueObject>(ResultObject);
		}

		ensureMsgf(ResultValue.IsValid(), TEXT("Map key type is invalid."));
	}

	return ResultValue;
}

TSharedPtr<FJsonValue> FEditorConfig::WriteValue(const FProperty* Property, const void* DataPtr, const void* DefaultPtr)
{
	TSharedPtr<FJsonValue> ResultValue;
	if (DefaultPtr != nullptr && Property->Identical(DataPtr, DefaultPtr))
	{
		return ResultValue;
	}

	if (const FStrProperty* StrProperty = CastField<FStrProperty>(Property))
	{
		FString* Value = (FString*) DataPtr;
		ResultValue = MakeShared<FJsonValueString>(*Value);
	}
	else if (const FNameProperty* NameProperty = CastField<FNameProperty>(Property))
	{
		FName* Value = (FName*) DataPtr;
		ResultValue = MakeShared<FJsonValueString>(Value->ToString());
	}
	else if (const FTextProperty* TextProperty = CastField<FTextProperty>(Property))
	{
		FText* Value = (FText*) DataPtr;
		ResultValue = MakeShared<FJsonValueString>(Value->ToString());
	}
	else if (const FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
	{
		bool Value = BoolProperty->GetPropertyValue(DataPtr);
		ResultValue = MakeShared<FJsonValueBoolean>(Value);
	}
	else if (const FFloatProperty* FloatProperty = CastField<FFloatProperty>(Property))
	{
		float* Value = (float*) DataPtr;
		ResultValue = MakeShared<FJsonValueNumber>(*Value);
	}
	else if (const FDoubleProperty* DoubleProperty = CastField<FDoubleProperty>(Property))
	{
		double* Value = (double*) DataPtr;
		ResultValue = MakeShared<FJsonValueNumber>(*Value);
	} 
	else if (const FInt8Property* Int8Property = CastField<FInt8Property>(Property))
	{
		int8* Value = (int8*) DataPtr;
		ResultValue = MakeShared<FJsonValueNumber>(static_cast<double>(*Value));
	}
	else if (const FInt16Property* Int16Property = CastField<FInt16Property>(Property))
	{
		int16* Value = (int16*) DataPtr;
		ResultValue = MakeShared<FJsonValueNumber>(static_cast<double>(*Value));
	}
	else if (const FIntProperty* Int32Property = CastField<FIntProperty>(Property))
	{
		int32* Value = (int32*) DataPtr;
		ResultValue = MakeShared<FJsonValueNumber>(static_cast<double>(*Value));
	}
	else if (const FInt64Property* Int64Property = CastField<FInt64Property>(Property))
	{
		int64* Value = (int64*) DataPtr;
		ResultValue = MakeShared<FJsonValueNumber>(static_cast<double>(*Value));
	}
	else if (const FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
	{
		uint8* Value = (uint8*) DataPtr;
		ResultValue = MakeShared<FJsonValueNumber>(static_cast<double>(*Value));
	}
	else if (const FUInt16Property* Uint16Property = CastField<FUInt16Property>(Property))
	{
		uint16* Value = (uint16*) DataPtr;
		ResultValue = MakeShared<FJsonValueNumber>(static_cast<double>(*Value));
	}
	else if (const FUInt32Property* Uint32Property = CastField<FUInt32Property>(Property))
	{
		uint32* Value = (uint32*) DataPtr;
		ResultValue = MakeShared<FJsonValueNumber>(static_cast<double>(*Value));
	}
	else if (const FUInt64Property* Uint64Property = CastField<FUInt64Property>(Property))
	{
		uint64* Value = (uint64*) DataPtr;
		ResultValue = MakeShared<FJsonValueNumber>(static_cast<double>(*Value));
	}
	else if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
	{
		UEnum* Enum = EnumProperty->GetEnum();
		int64 Value = EnumProperty->GetUnderlyingProperty()->GetSignedIntPropertyValue(DataPtr);
		FName ValueName = Enum->GetNameByValue(Value);
		ResultValue = MakeShared<FJsonValueString>(ValueName.ToString());
	}
	else if (const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(Property))
	{
		FString ObjectPath;
		ObjectProperty->ExportTextItem_Direct(ObjectPath, DataPtr, nullptr, nullptr, PPF_None, nullptr);
		ResultValue = MakeShared<FJsonValueString>(ObjectPath);
	}
	else if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
	{
		ResultValue = MakeShared<FJsonValueObject>(WriteStruct(StructProperty->Struct, DataPtr, DefaultPtr, EPropertyFilter::All));
	}
	else if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
	{
		ResultValue = WriteArray(ArrayProperty, DataPtr);
	}
	else if (const FSetProperty* SetProperty = CastField<FSetProperty>(Property))
	{
		ResultValue = WriteSet(SetProperty, DataPtr);
	}
	else if (const FMapProperty* MapProperty = CastField<FMapProperty>(Property))
	{
		ResultValue = WriteMap(MapProperty, DataPtr);
	}

	ensureAlwaysMsgf(ResultValue.IsValid(), TEXT("Property type is unsupported: %s, type: %s"), *Property->GetPathName(), *Property->GetClass()->GetName());
	return ResultValue;
}

TSharedPtr<FJsonObject> FEditorConfig::WriteStruct(const UStruct* Struct, const void* Instance, const void* Defaults, EPropertyFilter Filter) 
{
	TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
	JsonObject->SetStringField(TEXT("$type"), Struct->GetName());
	
	// Default initialize a struct here if we weren't passed one. 
	// This is necessary because structs can contain default initializations, eg.:
	// struct Foo { float Bar = 5.0f; }
	// The Bar FProperty does not store this value itself, only the struct does.
	TArray<uint8> TempDefaults;

	if (Defaults == nullptr)
	{
		TempDefaults.AddZeroed(Struct->GetStructureSize());
		Struct->InitializeStruct(TempDefaults.GetData());

		Defaults = TempDefaults.GetData();
	}

	bool bAnyWritten = false;

	for (TFieldIterator<FProperty> It(Struct); It; ++It)
	{
		const FProperty* Property = *It;

		if (Filter == EPropertyFilter::MetadataOnly && !Property->HasMetaData("EditorConfig"))
		{
			continue;
		}

		bAnyWritten = true;

		const void* ValuePtr = Property->ContainerPtrToValuePtr<void>(Instance);
		const void* PropertyDefaultPtr = Property->ContainerPtrToValuePtr<void>(Defaults);

		TSharedPtr<FJsonValue> PropertyValue = WriteValue(Property, ValuePtr, PropertyDefaultPtr);
		if (PropertyValue.IsValid())
		{
			JsonObject->SetField(Property->GetName(), PropertyValue);
		}
	}

	ensureAlwaysMsgf(bAnyWritten, TEXT("Struct type has no properties to serialize: %s"), *Struct->GetName());

	return JsonObject;
}

/** 
 * This exists because of sparse class data that can exist for UObjects only, which is handled in ContainerPtrToValuePtr.
 */
TSharedPtr<FJsonObject> FEditorConfig::WriteUObject(const UClass* Class, const UObject* Instance, EPropertyFilter Filter) 
{
	TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
	JsonObject->SetStringField(TEXT("$type"), Class->GetName());

	const UObject* Defaults = Class->GetDefaultObject();

	bool bAnyWritten = false;

	for (TFieldIterator<FProperty> It(Class); It; ++It)
	{
		const FProperty* Property = *It;

		if (Filter == EPropertyFilter::MetadataOnly && !Property->HasMetaData("EditorConfig"))
		{
			continue;
		}

		bAnyWritten = true;

		const void* ValuePtr = Property->ContainerPtrToValuePtr<void>(Instance);
		const void* PropertyDefaultPtr = Property->ContainerPtrToValuePtr<void>(Defaults);

		TSharedPtr<FJsonValue> PropertyValue = WriteValue(Property, ValuePtr, PropertyDefaultPtr);
		if (PropertyValue.IsValid())
		{
			JsonObject->SetField(Property->GetName(), PropertyValue);
		}
	}

	ensureAlwaysMsgf(bAnyWritten, TEXT("UObject type has no properties to serialize: %s"), *Class->GetName());

	return JsonObject;
}

void FEditorConfig::SetDirty()
{
	if (!Dirty)
	{
		Dirty = true;
		EditorConfigDirtiedEvent.Broadcast(*this);
	}
}

void FEditorConfig::OnSaved()
{
	Dirty = false;
}
