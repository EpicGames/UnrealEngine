// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorConfigBase.h"

#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "EditorConfigSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorConfigBase)

bool UEditorConfigBase::LoadEditorConfig()
{
	if (GEditor)
	{
		UEditorConfigSubsystem* Subsystem = GEditor->GetEditorSubsystem<UEditorConfigSubsystem>();
		if (Subsystem)
		{
			return Subsystem->LoadConfigObject(this->GetClass(), this);
		}
	}
	return false;
}

bool UEditorConfigBase::SaveEditorConfig() const
{
	if (GEditor)
	{
		UEditorConfigSubsystem* Subsystem = GEditor->GetEditorSubsystem<UEditorConfigSubsystem>();
		if (Subsystem)
		{
			return Subsystem->SaveConfigObject(this->GetClass(), this);
		}
	}
	return false;
}
