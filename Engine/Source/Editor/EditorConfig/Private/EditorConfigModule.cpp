// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorConfigModule.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogEditorConfig);

IMPLEMENT_MODULE(FEditorConfigModule, EditorConfig);
