// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorConfigSubsystem.h"

#include "Async/Async.h"
#include "Editor.h"
#include "Misc/App.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"

TArray<TPair<UEditorConfigSubsystem::ESearchDirectoryType, FString>> UEditorConfigSubsystem::EarlyRegistredSearchDirectories;

UEditorConfigSubsystem::UEditorConfigSubsystem()
{
	
}

void UEditorConfigSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	SearchDirectories.Reserve(4  + EarlyRegistredSearchDirectories.Num());

	AddSearchDirectory(ESearchDirectoryType::Engine, FPaths::Combine(FPaths::EngineConfigDir(), TEXT("Editor"))); // Engine
	AddSearchDirectory(ESearchDirectoryType::Project, FPaths::Combine(FPaths::ProjectConfigDir(), TEXT("Editor"))); // ProjectName
#ifdef UE_SAVED_DIR_OVERRIDE
	AddSearchDirectory(ESearchDirectoryType::User, FPaths::Combine(FPlatformProcess::UserSettingsDir(), TEXT(UE_STRINGIZE(UE_SAVED_DIR_OVERRIDE)), TEXT("Editor"))); // AppData
#else
	AddSearchDirectory(ESearchDirectoryType::User, FPaths::Combine(FPlatformProcess::UserSettingsDir(), *FApp::GetEpicProductIdentifier(), TEXT("Editor"))); // AppData
#endif

	for (const TPair<ESearchDirectoryType, FString>& Pair : EarlyRegistredSearchDirectories)
	{
		AddSearchDirectory(Pair.Key, Pair.Value);
	}

	EarlyRegistredSearchDirectories.Empty();
}

void UEditorConfigSubsystem::Deinitialize()
{
	FScopeLock Lock(&SaveLock);

	// Synchronously save all Pending Saves on exit
	for (FPendingSave& Save : PendingSaves)
	{
		const FString* FilePath = LoadedConfigs.FindKey(Save.Config);
		check(FilePath != nullptr);

		Save.Config->SaveToFile(*FilePath);
		Save.Config->OnSaved();
	}
}

void UEditorConfigSubsystem::Tick(float DeltaTime)
{
	FScopeLock Lock(&SaveLock);

	// Allows PendingSaves to be modified while iterating as
	// the Async below might execute the task immediately
	// when running in -nothreading mode.
	for (int32 Index = 0; Index < PendingSaves.Num(); ++Index)
	{
		FPendingSave& Save = PendingSaves[Index];
		Save.TimeSinceQueued += DeltaTime;

		const float SaveDelaySeconds = 3.0f;
		if (Save.TimeSinceQueued > SaveDelaySeconds && 
			!Save.WasSuccess.IsValid())
		{
			const FString* FilePath = LoadedConfigs.FindKey(Save.Config);
			check(FilePath != nullptr);
			
			Save.WasSuccess = Async(EAsyncExecution::TaskGraph, 
				[Config = Save.Config, File = *FilePath]()
				{
					return Config->SaveToFile(File);
				}, 
				[this, Config = Save.Config]()
				{
					OnSaveCompleted(Config);
				});
		}
	}

	for (int32 Index = 0; Index < PendingSaves.Num(); ++Index)
	{
		if (PendingSaves[Index].WasSuccess.IsReady())
		{
			PendingSaves.RemoveAt(Index);
			--Index;
		}
	}
}

TStatId UEditorConfigSubsystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UEditorConfigSubsystem, STATGROUP_Tickables);
}

bool UEditorConfigSubsystem::LoadConfigObject(const UClass* Class, UObject* Object, FEditorConfig::EPropertyFilter Filter)
{
	const FString& EditorConfigName = Class->GetMetaData("EditorConfig");
	if (!ensureMsgf(!EditorConfigName.IsEmpty(), TEXT("UEditorConfigSubsystem::LoadConfigObject - EditorConfig name is not set on class %s."), *Class->GetName()))
	{
		return false;
	}

	TSharedRef<FEditorConfig> EditorConfig = FindOrLoadConfig(EditorConfigName);
	return EditorConfig->TryGetRootUObject(Class, Object, Filter);
}

bool UEditorConfigSubsystem::SaveConfigObject(const UClass* Class, const UObject* Object, FEditorConfig::EPropertyFilter Filter)
{
	const FString& EditorConfigName = Class->GetMetaData("EditorConfig");
	if (!ensureMsgf(!EditorConfigName.IsEmpty(), TEXT("UEditorConfigSubsystem::SaveConfigObject - EditorConfig name is not set on class %s."), *Class->GetName()))
	{
		return false;
	}

	TSharedRef<FEditorConfig> EditorConfig = FindOrLoadConfig(EditorConfigName);
	EditorConfig->SetRootUObject(Class, Object, Filter);
	SaveConfig(EditorConfig);
	return true;
}

bool UEditorConfigSubsystem::ReloadConfig(TSharedRef<FEditorConfig> Config)
{
	const FString* FilePath = LoadedConfigs.FindKey(Config);
	if (!ensureMsgf(FilePath != nullptr, TEXT("Could not find filename for given config in UEditorConfigSubsystem::ReloadConfig().")))
	{
		return false;
	}

	const FString ConfigName = FPaths::GetBaseFilename(*FilePath);

	TSharedPtr<FEditorConfig> Parent;

	for (const TPair<ESearchDirectoryType, FString>& Dir : SearchDirectories)
	{
		const FString FullPath = FPaths::Combine(Dir.Value, ConfigName) + TEXT(".json");

		// find an existing config or create one
		const TSharedPtr<FEditorConfig>* Existing = LoadedConfigs.Find(FullPath);
		if (Existing == nullptr)
		{
			Existing = &LoadedConfigs.Add(FullPath, MakeShared<FEditorConfig>());
		}

		if (!(*Existing)->LoadFromFile(FullPath))
		{
			ensureMsgf(false, TEXT("Failed to load editor config from file %s"), *FullPath);
			return false;
		}

		if (Parent.IsValid())
		{
			(*Existing)->SetParent(Parent);
		}

		Parent = (*Existing);
	}

	return true;
}

TSharedRef<FEditorConfig> UEditorConfigSubsystem::FindOrLoadConfig(FStringView ConfigName, ESearchDirectoryType IncludedTypes)
{
	checkf(!ConfigName.IsEmpty(), TEXT("Config name cannot be empty!"));

	FString ConfigNameString(ConfigName);

	// Construct a subset of SearchDirectories to be searched based on IncludedTypes
	TArrayView<TPair<ESearchDirectoryType, FString>> IncludedSearchDirectories = SearchDirectories;
	int32 LastSearchDirectoryIndex = SearchDirectories.Num() - 1;
	for (; LastSearchDirectoryIndex >= 0; --LastSearchDirectoryIndex)
	{
		if (SearchDirectories[LastSearchDirectoryIndex].Key <= IncludedTypes)
		{
			break;
		}
	}
	checkf(LastSearchDirectoryIndex > 0, TEXT("No valid search directories found for IncludedTypes = %i"), (int32)IncludedTypes);
	IncludedSearchDirectories.LeftInline(LastSearchDirectoryIndex + 1);
	
	// look for the config in the final search directory and return if it's loaded
	// this assumes that the hierarchy of configs is unchanged
	// ie. given search directories [Foo, Bar], the existence of Bar/X.json is taken to mean that Foo/X.json has been loaded
	const FString FinalPath = FPaths::Combine(IncludedSearchDirectories.Last().Value, ConfigNameString) + TEXT(".json");

	const TSharedPtr<FEditorConfig>* FinalConfig = LoadedConfigs.Find(FinalPath);
	if (FinalConfig != nullptr)
	{
		return FinalConfig->ToSharedRef();
	}
	
	// find or load all configs in all search directories with the given name
	TSharedPtr<FEditorConfig> Parent;

	for (const TPair<ESearchDirectoryType, FString>& Dir : IncludedSearchDirectories)
	{
		const FString FullPath = FPaths::Combine(Dir.Value, ConfigNameString) + TEXT(".json");

		const TSharedPtr<FEditorConfig>* Existing = LoadedConfigs.Find(FullPath);
		if (Existing != nullptr && Existing->IsValid())
		{
			Parent = *Existing;
		}
		else
		{
			// didn't exist yet, load now
			TSharedRef<FEditorConfig> NewConfig = MakeShared<FEditorConfig>();
			if (NewConfig->LoadFromFile(FullPath))
			{
				NewConfig->OnEditorConfigDirtied().AddUObject(this, &UEditorConfigSubsystem::OnEditorConfigDirtied);

				if (Parent.IsValid())
				{
					NewConfig->SetParent(Parent);
				}

				LoadedConfigs.Add(FullPath, NewConfig);

				Parent = NewConfig;
			}
		}
	}

	FinalConfig = LoadedConfigs.Find(FinalPath);
	if (FinalConfig != nullptr)
	{
		return FinalConfig->ToSharedRef();
	}

	// no config in the last search directory, create one now
	// this will be the config that changes are written to
	TSharedRef<FEditorConfig> NewConfig = MakeShared<FEditorConfig>();
	NewConfig->OnEditorConfigDirtied().AddUObject(this, &UEditorConfigSubsystem::OnEditorConfigDirtied);

	if (Parent.IsValid())
	{
		// parent to the previous config
		NewConfig->SetParent(Parent);
	}

	LoadedConfigs.Add(FinalPath, NewConfig);

	return NewConfig;
}

void UEditorConfigSubsystem::OnEditorConfigDirtied(const FEditorConfig& Config)
{
	for (const TPair<FString, TSharedPtr<FEditorConfig>>& Pair : LoadedConfigs)
	{
		if (Pair.Value.Get() == &Config)
		{
			SaveConfig(Pair.Value.ToSharedRef());
		}
	}
}

void UEditorConfigSubsystem::SaveConfig(TSharedRef<FEditorConfig> Config)
{
	const FString* FilePath = LoadedConfigs.FindKey(Config);
	if (!ensureMsgf(FilePath != nullptr, TEXT("Saving config that was not loaded through FEditorConfigSubsystem::FindOrLoadConfig. System does not know filepath to save to.")))
	{
		return;
	}

	FScopeLock Lock(&SaveLock);

	FPendingSave* Existing = PendingSaves.FindByPredicate([Config](const FPendingSave& Element)
		{
			return Element.Config == Config;
		});

	if (Existing != nullptr)
	{
		// reset the timer if we're saving within the grace period and no save is already being executed
		if (!Existing->WasSuccess.IsValid())
		{
			Existing->TimeSinceQueued = 0;
		}
	}
	else
	{
		FPendingSave& NewSave = PendingSaves.AddDefaulted_GetRef();
		NewSave.Config = Config;
		NewSave.FileName = *FilePath;
		NewSave.TimeSinceQueued = 0;
	}
}

void UEditorConfigSubsystem::OnSaveCompleted(TSharedPtr<FEditorConfig> Config)
{
	FScopeLock Lock(&SaveLock);

	const int32 Index = PendingSaves.IndexOfByPredicate([Config](const FPendingSave& Element)
		{
			return Element.Config == Config;
		});

	if (Index != INDEX_NONE)
	{
		const FPendingSave& PendingSave = PendingSaves[Index];
		PendingSave.Config->OnSaved();
	}
}

void UEditorConfigSubsystem::AddSearchDirectory(ESearchDirectoryType Type, FStringView SearchDir)
{
	TPair<ESearchDirectoryType, FString> NewEntry(Type, SearchDir);
	if (!SearchDirectories.Contains(NewEntry))
	{
		for (int32 i = 0; i < SearchDirectories.Num(); ++i)
		{
			if (SearchDirectories[i].Key > Type)
			{
				SearchDirectories.Insert(NewEntry, i);
				return;
			}
		}
		SearchDirectories.Add(NewEntry);
	}
}

void UEditorConfigSubsystem::EarlyAddSearchDirectory(ESearchDirectoryType Type, FStringView SearchDir)
{
	EarlyRegistredSearchDirectories.Emplace(Type, SearchDir);
	
	check(!GEditor || !GEditor->IsInitialized());
}