// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorMetadataOverrides.h"

#include "Containers/StringView.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "EditorConfig.h"
#include "EditorConfigSubsystem.h"
#include "Misc/AssertionMacros.h"
#include "Subsystems/SubsystemCollection.h"
#include "UObject/Class.h"
#include "UObject/Field.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorMetadataOverrides)

UEditorMetadataOverrides::UEditorMetadataOverrides()
{
}

void UEditorMetadataOverrides::Initialize(FSubsystemCollectionBase& Collection)
{
	UEditorConfigSubsystem* EditorConfig = Collection.InitializeDependency<UEditorConfigSubsystem>();
	
	TSharedPtr<FEditorConfig> MetadataOverrideConfig = EditorConfig->FindOrLoadConfig(TEXT("MetadataOverrides"));
	LoadFromConfig(MetadataOverrideConfig);
}

bool UEditorMetadataOverrides::LoadFromConfig(TSharedPtr<FEditorConfig> Config)
{
	SourceConfig = Config;
	LoadedMetadata.Classes.Reset();

	if (!SourceConfig.IsValid())
	{
		return false;
	}

	return SourceConfig->TryGetStruct(TEXT("Metadata"), LoadedMetadata, FEditorConfig::EPropertyFilter::All);
}

void UEditorMetadataOverrides::Save()
{
	if (!SourceConfig.IsValid())
	{
		return;
	}

	SourceConfig->SetStruct(TEXT("Metadata"), LoadedMetadata, FEditorConfig::EPropertyFilter::All);

	UEditorConfigSubsystem* EditorConfigSubsystem = GEditor->GetEditorSubsystem<UEditorConfigSubsystem>();
	if (EditorConfigSubsystem == nullptr)
	{
		return;
	}

	EditorConfigSubsystem->SaveConfig(SourceConfig.ToSharedRef());
}

const FMetadataSet* UEditorMetadataOverrides::FindFieldMetadata(const FField* Field) const
{
	check(Field != nullptr);
	
	if (!SourceConfig.IsValid())
	{
		return nullptr;
	}

	const UStruct* OwnerStruct = Field->GetOwnerStruct();
	if (OwnerStruct == nullptr)
	{
		return nullptr;
	}

	const FStructMetadata* StructMetadata = LoadedMetadata.Classes.Find(OwnerStruct->GetFName());
	if (StructMetadata == nullptr)
	{
		return nullptr;
	}

	const FMetadataSet* FieldMetadata = StructMetadata->Fields.Find(Field->GetFName());
	return FieldMetadata;
}

FMetadataSet* UEditorMetadataOverrides::FindFieldMetadata(const FField* Field)
{
	check(Field != nullptr);
	
	if (!SourceConfig.IsValid())
	{
		return nullptr;
	}

	const UStruct* OwnerStruct = Field->GetOwnerStruct();
	if (OwnerStruct == nullptr)
	{
		return nullptr;
	}

	FStructMetadata* StructMetadata = LoadedMetadata.Classes.Find(OwnerStruct->GetFName());
	if (StructMetadata == nullptr)
	{
		return nullptr;
	}

	FMetadataSet* FieldMetadata = StructMetadata->Fields.Find(Field->GetFName());
	return FieldMetadata;
}
	
FMetadataSet* UEditorMetadataOverrides::FindOrAddFieldMetadata(const FField* Field)
{
	check(Field != nullptr);

	if (!SourceConfig.IsValid())
	{
		return nullptr;
	}

	const UStruct* OwnerStruct = Field->GetOwnerStruct();
	if (OwnerStruct == nullptr)
	{
		return nullptr;
	}

	FStructMetadata& StructMetadata = LoadedMetadata.Classes.FindOrAdd(OwnerStruct->GetFName());

	FMetadataSet& FieldMetadata = StructMetadata.Fields.FindOrAdd(Field->GetFName());
	return &FieldMetadata;
}

EMetadataType UEditorMetadataOverrides::GetMetadataType(const FField* Field, FName Key) const
{
	check(Field != nullptr);

	const FMetadataSet* MetadataSet = FindFieldMetadata(Field);
	if (MetadataSet == nullptr)
	{
		return EMetadataType::None;
	}

	if (MetadataSet->Bools.Contains(Key))
	{
		return EMetadataType::Bool;
	}
	if (MetadataSet->Ints.Contains(Key))
	{
		return EMetadataType::Int;
	}
	if (MetadataSet->Floats.Contains(Key))
	{
		return EMetadataType::Float;
	}
	if (MetadataSet->Strings.Contains(Key))
	{
		return EMetadataType::String;
	}

	return EMetadataType::None;
}

EMetadataType UEditorMetadataOverrides::GetMetadataType(const UStruct* Struct, FName Key) const
{
	check(Struct != nullptr);

	const FMetadataSet* MetadataSet = FindStructMetadata(Struct);
	if (MetadataSet == nullptr)
	{
		return EMetadataType::None; 
	}

	if (MetadataSet->Bools.Contains(Key))
	{
		return EMetadataType::Bool;
	}
	if (MetadataSet->Ints.Contains(Key))
	{
		return EMetadataType::Int;
	}
	if (MetadataSet->Floats.Contains(Key))
	{
		return EMetadataType::Float;
	}
	if (MetadataSet->Strings.Contains(Key))
	{
		return EMetadataType::String;
	}

	return EMetadataType::None;
}

bool UEditorMetadataOverrides::GetStringMetadata(const FField* Field, FName Key, FString& OutValue) const
{
	const FMetadataSet* FieldMetadata = FindFieldMetadata(Field);
	if (FieldMetadata == nullptr)
	{
		return false;
	}

	const FString* MetaValue = FieldMetadata->Strings.Find(Key);
	if (MetaValue == nullptr)
	{
		if (!Field->HasMetaData(Key))
		{
			return false;
		}

		OutValue = Field->GetMetaData(Key);
		return true;
	}

	OutValue = *MetaValue;
	return true;
}

void UEditorMetadataOverrides::SetStringMetadata(const FField* Field, FName Key, FStringView Value)
{
	FMetadataSet* FieldMetadata = FindOrAddFieldMetadata(Field);
	if (FieldMetadata == nullptr)
	{
		return;
	}

	FieldMetadata->Strings.Add(Key, FString(Value));
}

bool UEditorMetadataOverrides::GetFloatMetadata(const FField* Field, FName Key, float& OutValue) const
{
	const FMetadataSet* FieldMetadata = FindFieldMetadata(Field);
	if (FieldMetadata == nullptr)
	{
		return false;
	}

	const float* MetaValue = FieldMetadata->Floats.Find(Key);
	if (MetaValue == nullptr)
	{
		if (!Field->HasMetaData(Key))
		{
			return false;
		}

		OutValue = Field->GetFloatMetaData(Key);
		return true;
	}

	OutValue = *MetaValue;
	return true;
}

void UEditorMetadataOverrides::SetFloatMetadata(const FField* Field, FName Key, float Value)
{
	FMetadataSet* FieldMetadata = FindOrAddFieldMetadata(Field);
	if (FieldMetadata == nullptr)
	{
		return;
	}

	FieldMetadata->Floats.Add(Key, Value);
	Save();
}

bool UEditorMetadataOverrides::GetIntMetadata(const FField* Field, FName Key, int32& OutValue) const
{
	const FMetadataSet* FieldMetadata = FindFieldMetadata(Field);
	if (FieldMetadata == nullptr)
	{
		return false;
	}

	const int32* MetaValue = FieldMetadata->Ints.Find(Key);
	if (MetaValue == nullptr)
	{
		if (!Field->HasMetaData(Key))
		{
			return false;
		}

		OutValue = Field->GetIntMetaData(Key);
		return true;
	}

	OutValue = *MetaValue;
	return true;
}

void UEditorMetadataOverrides::SetIntMetadata(const FField* Field, FName Key, int32 Value)
{
	FMetadataSet* FieldMetadata = FindOrAddFieldMetadata(Field);
	if (FieldMetadata == nullptr)
	{
		return;
	}

	FieldMetadata->Ints.Add(Key, Value);
	Save();
}

bool UEditorMetadataOverrides::GetBoolMetadata(const FField* Field, FName Key, bool& OutValue) const
{
	const FMetadataSet* FieldMetadata = FindFieldMetadata(Field);
	if (FieldMetadata == nullptr)
	{
		return false;
	}

	const bool* MetaValue = FieldMetadata->Bools.Find(Key);
	if (MetaValue == nullptr)
	{
		if (!Field->HasMetaData(Key))
		{
			return false;
		}

		OutValue = Field->GetBoolMetaData(Key);
		return true;
	}

	OutValue = *MetaValue;
	return true;
}
	
void UEditorMetadataOverrides::SetBoolMetadata(const FField* Field, FName Key, bool Value)
{
	FMetadataSet* FieldMetadata = FindOrAddFieldMetadata(Field);
	if (FieldMetadata == nullptr)
	{
		return;
	}

	FieldMetadata->Bools.Add(Key, Value);
	Save();
}

bool UEditorMetadataOverrides::GetClassMetadata(const FField* Field, FName Key, UClass*& OutValue) const
{
	FString ClassName;
	if (!GetStringMetadata(Field, Key, ClassName))
	{
		return false;
	}

	OutValue = UClass::TryFindTypeSlow<UClass>(ClassName);
	return true; // we return true here even if the value is null because we did have a value, it just wasn't a valid class name
}

void UEditorMetadataOverrides::SetClassMetadata(const FField* Field, FName Key, UClass* Value)
{
	FString ClassName;
	if (Value != nullptr)
	{
		ClassName = Value->GetName();
	}

	SetStringMetadata(Field, Key, ClassName);
	Save();
}

bool UEditorMetadataOverrides::GetArrayMetadata(const FField* Field, FName Key, TArray<FString>& OutValue) const
{
	const FMetadataSet* FieldMetadata = FindFieldMetadata(Field);
	if (FieldMetadata == nullptr)
	{
		return false;
	}

	const FString* MetaValue = FieldMetadata->Strings.Find(Key);
	if (MetaValue != nullptr)
	{
		MetaValue->ParseIntoArray(OutValue, TEXT(","));
		return true;
	}

	// fallback to metadata system
	if (!Field->HasMetaData(Key))
	{
		return false;
	}

	const FString ArrayString = Field->GetMetaData(Key);
	ArrayString.ParseIntoArray(OutValue, TEXT(","));
	return true;
}
	
void UEditorMetadataOverrides::SetArrayMetadata(const FField* Field, FName Key, const TArray<FString>& Value)
{
	FMetadataSet* FieldMetadata = FindOrAddFieldMetadata(Field);
	if (FieldMetadata == nullptr)
	{
		return;
	}

	const FString ValueString = FString::Join(Value, TEXT(","));
	FieldMetadata->Strings.Add(Key, ValueString);
	Save();
}

void UEditorMetadataOverrides::AddToArrayMetadata(const FField* Field, FName Key, const FString& Value)
{
	FMetadataSet* FieldMetadata = FindOrAddFieldMetadata(Field);
	if (FieldMetadata == nullptr)
	{
		return;
	}

	FString& CurrentValue = FieldMetadata->Strings.FindOrAdd(Key);
	if (CurrentValue.IsEmpty())
	{
		CurrentValue = Value;
	}
	else
	{
		CurrentValue.Append(TEXT(","));
		CurrentValue.Append(Value);
	}
	Save();
}

void UEditorMetadataOverrides::RemoveFromArrayMetadata(const FField* Field, FName Key, const FString& Value)
{
	FMetadataSet* FieldMetadata = FindFieldMetadata(Field);
	if (FieldMetadata == nullptr)
	{
		return;
	}

	FString* CurrentValue = FieldMetadata->Strings.Find(Key);
	if (CurrentValue == nullptr)
	{
		return;
	}
	
	TArray<FString> ArrayValue;
	CurrentValue->ParseIntoArray(ArrayValue, TEXT(","));

	ArrayValue.Remove(Value);

	if (ArrayValue.IsEmpty())
	{
		FieldMetadata->Strings.Remove(Key);
	}
	else
	{
		const FString ValueString = FString::Join(ArrayValue, TEXT(","));
		FieldMetadata->Strings.Add(Key, ValueString);
	}

	Save();
}

void UEditorMetadataOverrides::RemoveMetadata(const FField* Field, FName Key)
{
	if (!SourceConfig.IsValid())
	{
		return;
	}

	const UStruct* OwnerStruct = Field->GetOwnerStruct();
	if (OwnerStruct == nullptr)
	{
		return;
	}

	FStructMetadata* StructMetadata = LoadedMetadata.Classes.Find(OwnerStruct->GetFName());
	if (StructMetadata == nullptr)
	{
		return;
	}

	FMetadataSet* FieldMetadata = StructMetadata->Fields.Find(Field->GetFName());
	if (FieldMetadata == nullptr)
	{
		return;
	}

	FieldMetadata->Ints.Remove(Key);
	FieldMetadata->Bools.Remove(Key);
	FieldMetadata->Floats.Remove(Key);
	FieldMetadata->Strings.Remove(Key);

	Save();
}

const FMetadataSet* UEditorMetadataOverrides::FindStructMetadata(const UStruct* Struct) const
{
	check(Struct != nullptr);

	if (!SourceConfig.IsValid())
	{
		return nullptr;
	}

	const FStructMetadata* StructMetadata = LoadedMetadata.Classes.Find(Struct->GetFName());
	return StructMetadata != nullptr ? &StructMetadata->StructMetadata : nullptr;
}

FMetadataSet* UEditorMetadataOverrides::FindStructMetadata(const UStruct* Struct)
{
	check(Struct != nullptr);

	if (!SourceConfig.IsValid())
	{
		return nullptr;
	}

	FStructMetadata* StructMetadata = LoadedMetadata.Classes.Find(Struct->GetFName());
	return StructMetadata != nullptr ? &StructMetadata->StructMetadata : nullptr;
}
	
FMetadataSet* UEditorMetadataOverrides::FindOrAddStructMetadata(const UStruct* Struct)
{
	check(Struct != nullptr);

	if (!SourceConfig.IsValid())
	{
		return nullptr;
	}

	FStructMetadata& StructMetadata = LoadedMetadata.Classes.FindOrAdd(Struct->GetFName());
	return &StructMetadata.StructMetadata;
}

bool UEditorMetadataOverrides::GetStringMetadata(const UStruct* Struct, FName Key, FString& OutValue) const
{
	const FMetadataSet* StructMetadata = FindStructMetadata(Struct);
	if (StructMetadata == nullptr)
	{
		return false;
	}

	const FString* MetaValue = StructMetadata->Strings.Find(Key);
	if (MetaValue == nullptr)
	{
		if (!Struct->HasMetaData(Key))
		{
			return false;
		}

		OutValue = Struct->GetMetaData(Key);
		return true;
	}

	OutValue = *MetaValue;
	return true;
}

void UEditorMetadataOverrides::SetStringMetadata(const UStruct* Struct, FName Key, FStringView Value)
{
	FMetadataSet* StructMetadata = FindOrAddStructMetadata(Struct);
	if (StructMetadata == nullptr)
	{
		return;
	}

	StructMetadata->Strings.Add(Key, FString(Value));
	Save();
}

bool UEditorMetadataOverrides::GetFloatMetadata(const UStruct* Struct, FName Key, float& OutValue) const
{
	const FMetadataSet* StructMetadata = FindStructMetadata(Struct);
	if (StructMetadata == nullptr)
	{
		return false;
	}

	const float* MetaValue = StructMetadata->Floats.Find(Key);
	if (MetaValue == nullptr)
	{
		if (!Struct->HasMetaData(Key))
		{
			return false;
		}

		OutValue = Struct->GetFloatMetaData(Key);
		return true;
	}

	OutValue = *MetaValue;
	return true;
}

void UEditorMetadataOverrides::SetFloatMetadata(const UStruct* Struct, FName Key, float Value)
{
	FMetadataSet* StructMetadata = FindOrAddStructMetadata(Struct);
	if (StructMetadata == nullptr)
	{
		return;
	}

	StructMetadata->Floats.Add(Key, Value);
	Save();
}

bool UEditorMetadataOverrides::GetIntMetadata(const UStruct* Struct, FName Key, int32& OutValue) const
{
	const FMetadataSet* StructMetadata = FindStructMetadata(Struct);
	if (StructMetadata == nullptr)
	{
		return false;
	}

	const int32* MetaValue = StructMetadata->Ints.Find(Key);
	if (MetaValue == nullptr)
	{
		if (!Struct->HasMetaData(Key))
		{
			return false;
		}

		OutValue = Struct->GetIntMetaData(Key);
		return true;
	}

	OutValue = *MetaValue;
	return true;
}

void UEditorMetadataOverrides::SetIntMetadata(const UStruct* Struct, FName Key, int32 Value)
{
	FMetadataSet* StructMetadata = FindOrAddStructMetadata(Struct);
	if (StructMetadata == nullptr)
	{
		return;
	}

	StructMetadata->Ints.Add(Key, Value);
	Save();
}

bool UEditorMetadataOverrides::GetBoolMetadata(const UStruct* Struct, FName Key, bool& OutValue) const
{
	const FMetadataSet* StructMetadata = FindStructMetadata(Struct);
	if (StructMetadata == nullptr)
	{
		return false;
	}

	const bool* MetaValue = StructMetadata->Bools.Find(Key);
	if (MetaValue == nullptr)
	{
		if (!Struct->HasMetaData(Key))
		{
			return false;
		}

		OutValue = Struct->GetBoolMetaData(Key);
		return true;
	}

	OutValue = *MetaValue;
	return true;
}
	
void UEditorMetadataOverrides::SetBoolMetadata(const UStruct* Struct, FName Key, bool Value)
{
	FMetadataSet* StructMetadata = FindOrAddStructMetadata(Struct);
	if (StructMetadata == nullptr)
	{
		return;
	}

	StructMetadata->Bools.Add(Key, Value);
	Save();
}

bool UEditorMetadataOverrides::GetClassMetadata(const UStruct* Struct, FName Key, UClass*& OutValue) const
{
	FString ClassName;
	if (!GetStringMetadata(Struct, Key, ClassName))
	{
		return false;
	}

	OutValue = UClass::TryFindTypeSlow<UClass>(ClassName);
	return true; // we return true here even if the class is null because we did have a value, it just wasn't a valid class name
}

void UEditorMetadataOverrides::SetClassMetadata(const UStruct* Struct, FName Key, UClass* Value)
{
	FString ClassName;
	if (Value != nullptr)
	{
		ClassName = Value->GetName();
	}

	SetStringMetadata(Struct, Key, ClassName);
	Save();
}

bool UEditorMetadataOverrides::GetArrayMetadata(const UStruct* Struct, FName Key, TArray<FString>& OutValue) const
{
	const FMetadataSet* StructMetadata = FindStructMetadata(Struct);
	if (StructMetadata == nullptr)
	{
		return false;
	}

	const FString* MetaValue = StructMetadata->Strings.Find(Key);
	if (MetaValue != nullptr)
	{
		MetaValue->ParseIntoArray(OutValue, TEXT(","));
		return true;
	}

	if (!Struct->HasMetaData(Key))
	{
		return false;
	}

	const FString ArrayString = Struct->GetMetaData(Key);
	ArrayString.ParseIntoArray(OutValue, TEXT(","));
	return true;
}
	
void UEditorMetadataOverrides::SetArrayMetadata(const UStruct* Struct, FName Key, const TArray<FString>& Value)
{
	FMetadataSet* StructMetadata = FindOrAddStructMetadata(Struct);
	if (StructMetadata == nullptr)
	{
		return;
	}

	const FString ValueString = FString::Join(Value, TEXT(","));

	StructMetadata->Strings.Add(Key, ValueString);
	Save();
}

void UEditorMetadataOverrides::AddToArrayMetadata(const UStruct* Struct, FName Key, const FString& Value)
{
	FMetadataSet* StructMetadata = FindOrAddStructMetadata(Struct);
	if (StructMetadata == nullptr)
	{
		return;
	}

	FString& CurrentValue = StructMetadata->Strings.FindOrAdd(Key);
	if (CurrentValue.IsEmpty())
	{
		CurrentValue = Value;
	}
	else
	{
		CurrentValue.Append(TEXT(","));
		CurrentValue.Append(Value);
	}

	Save();
}

void UEditorMetadataOverrides::RemoveFromArrayMetadata(const UStruct* Struct, FName Key, const FString& Value)
{
	FMetadataSet* StructMetadata = FindStructMetadata(Struct);
	if (StructMetadata == nullptr)
	{
		return;
	}

	FString* CurrentValue = StructMetadata->Strings.Find(Key);
	if (CurrentValue == nullptr)
	{
		return;
	}
	
	TArray<FString> ArrayValue;
	CurrentValue->ParseIntoArray(ArrayValue, TEXT(","));

	ArrayValue.Remove(Value);

	if (ArrayValue.IsEmpty())
	{
		StructMetadata->Strings.Remove(Key);
	}
	else
	{
		const FString ValueString = FString::Join(ArrayValue, TEXT(","));
		StructMetadata->Strings.Add(Key, ValueString);
	}

	Save();
}

void UEditorMetadataOverrides::RemoveMetadata(const UStruct* Struct, FName Key)
{
	if (!SourceConfig.IsValid())
	{
		return;
	}

	FStructMetadata* StructMetadata = LoadedMetadata.Classes.Find(Struct->GetFName());
	if (StructMetadata == nullptr)
	{
		return;
	}

	StructMetadata->StructMetadata.Ints.Remove(Key);
	StructMetadata->StructMetadata.Bools.Remove(Key);
	StructMetadata->StructMetadata.Floats.Remove(Key);
	StructMetadata->StructMetadata.Strings.Remove(Key);

	Save();
}
