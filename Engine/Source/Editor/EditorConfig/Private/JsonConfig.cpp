// Copyright Epic Games, Inc. All Rights Reserved.

#include "JsonConfig.h"

#include "Containers/StringView.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "EditorConfigModule.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Text.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/FileHelper.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonTypes.h"
#include "Serialization/JsonWriter.h"
#include "Templates/EnableIf.h"
#include "Templates/UnrealTemplate.h"
#include "Trace/Detail/Channel.h"
#include "UObject/NameTypes.h"

DEFINE_LOG_CATEGORY_STATIC(LogJsonConfig, Log, All);

namespace UE
{
	FJsonPath::FJsonPath()
	{
	}

	FJsonPath::FJsonPath(const TCHAR* InPath)
	{
		ParsePath(InPath);
	}

	FJsonPath::FJsonPath(FStringView InPath)
	{
		ParsePath(FString(InPath));
	}

	FJsonPath::FJsonPath(const FJsonPath& Other) :
		PathParts(Other.PathParts)
	{
	}

	FJsonPath::FJsonPath(FJsonPath&& Other) :
		PathParts(MoveTemp(Other.PathParts))
	{
	}

	void FJsonPath::ParsePath(const FString& InPath)
	{
		TArray<FString> PathStrings;
		InPath.ParseIntoArray(PathStrings, TEXT("."), false);

		// we don't really care about the document root, remove it if it exists
		if (PathStrings.Num() > 0 && PathStrings[0] == TEXT("$"))
		{
			PathStrings.RemoveAt(0);
		}

		for (const FString& Part : PathStrings)
		{
			if (Part.IsEmpty())
			{
				PathParts.Empty();
				UE_LOGF(LogJsonConfig, Warning, "Path part was empty in JSON path \"%ls\"", *InPath);
				return;
			}

			// check for index
			int32 OpenBracket, CloseBracket;
			if (Part.FindChar('[', OpenBracket))
			{
				if (Part.FindChar(']', CloseBracket) && 
					CloseBracket > OpenBracket &&
					CloseBracket == Part.Len() - 1)
				{
					FString Index = Part.Mid(OpenBracket + 1, (CloseBracket - OpenBracket) - 1);
					if (Index.IsNumeric())
					{
						FPart NewPart;
						NewPart.Name = *Part.Left(OpenBracket);
						NewPart.Index = FCString::Atoi(*Index);
					
						PathParts.Add(NewPart);
					}
					else
					{
						PathParts.Empty();
						UE_LOGF(LogJsonConfig, Warning, "Path part \"%ls\" looked to be index into array but was malformed in JSON path \"%ls\"", *Part, *InPath);
						return;
					}
				}
				else
				{
					PathParts.Empty();
					UE_LOGF(LogJsonConfig, Warning, "Path part \"%ls\" looked to be index into array but was malformed in JSON path \"%ls\"", *Part, *InPath);
					return;
				}
			}
			else
			{
				FPart NewPart;
				NewPart.Name = *Part;

				// basic path
				PathParts.Add(NewPart);
			}
		}
	}

	void FJsonPath::Append(FStringView Name)
	{
		FPart NewPart;
		NewPart.Name = Name;
		PathParts.Add(NewPart);
	}

	void FJsonPath::SetArrayIndex(int32 Index)
	{
		check(PathParts.Num() > 0);

		PathParts.Last().Index = Index;
	}

	FJsonPath FJsonPath::GetSubPath(int32 NumParts) const
	{
		FJsonPath SubPath;
		SubPath.PathParts.Reserve(NumParts);

		for (int32 Idx = 0; Idx < NumParts; ++Idx)
		{
			SubPath.PathParts.Add(PathParts[Idx]);
		}

		return MoveTemp(SubPath);
	}

	FString FJsonPath::ToString() const
	{
		TStringBuilder<256> StringBuilder;
		for (const FPart& Part : PathParts)
		{
			StringBuilder.Append(Part.Name);

			if (Part.Index != INDEX_NONE)
			{
				StringBuilder.Append(TEXT("["));
				StringBuilder.Append(LexToString(Part.Index));
				StringBuilder.Append(TEXT("]"));
			}
		}

		return StringBuilder.ToString();
	}


	//-----------------------------------------------------------------------------

	FJsonConfig::FJsonConfig()
	{
		OverrideObject = MakeShared<FJsonObject>();
		MergedObject = MakeShared<FJsonObject>();
	}

	void FJsonConfig::SetParent(const TSharedPtr<FJsonConfig>& Parent)
	{
		if (ParentConfig.IsValid())
		{
			ParentConfig->OnConfigChanged.Unbind();
		}

		ParentConfig = Parent;

		MergeThisWithParent();

		if (ParentConfig.IsValid())
		{
			ParentConfig->OnConfigChanged.BindRaw(this, &FJsonConfig::OnParentConfigChanged);
		}

		OnConfigChanged.ExecuteIfBound();
	}

	void FJsonConfig::OnParentConfigChanged()
	{
		MergeThisWithParent();
	}

	bool FJsonConfig::LoadFromFile(FStringView FilePath)
	{
		FString Contents;
		if (!FFileHelper::LoadFileToString(Contents, FilePath.GetData()))
		{
			return false;
		}

		if (!LoadFromString(Contents))
		{
			UE_LOGF(LogEditorConfig, Error, "Failed to load JSON file into JsonConfig %ls", FilePath.GetData());
			return false;
		}
		return true;
	}

	bool FJsonConfig::LoadFromString(FStringView Content)
	{
		TSharedRef<FJsonStringReader> JsonReader = FJsonStringReader::Create(FString(Content));
		if (!FJsonSerializer::Deserialize(JsonReader.Get(), OverrideObject))
		{
			UE_LOGF(LogEditorConfig, Error, "Failed to deserialize JSON string: %ls", *JsonReader->GetErrorMessage());
			return false;
		}

		if (!OverrideObject.IsValid())
		{
			return false;
		}

		OnConfigChanged.ExecuteIfBound();

		return MergeThisWithParent();
	}

	bool FJsonConfig::SaveToFile(FStringView FilePath) const
	{
		FString Contents;
		if (!SaveToString(Contents))
		{
			return false;
		}

		return FFileHelper::SaveStringToFile(Contents, FilePath.GetData());
	}

	bool FJsonConfig::SaveToString(FString& OutResult) const
	{
		if (!OverrideObject.IsValid())
		{
			return false;
		}

		TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutResult);
		if (!FJsonSerializer::Serialize(OverrideObject.ToSharedRef(), Writer.Get()))
		{
			return false;
		}

		return true;
	}

	static TSharedPtr<FJsonValue> GetKeyField(const TSharedPtr<FJsonValue>& Value)
	{
		if (Value->Type == EJson::Object)
		{
			return Value->AsObject()->TryGetField(TEXT("$key"));
		}

		return TSharedPtr<FJsonValue>();
	}

	static TSharedPtr<FJsonValue> GetValueField(const TSharedPtr<FJsonValue>& Value)
	{
		if (Value->Type == EJson::Object)
		{
			return Value->AsObject()->TryGetField(TEXT("$value"));
		}

		return TSharedPtr<FJsonValue>();
	}

	static TSharedPtr<FJsonValue> FindValueInArray(const TArray<TSharedPtr<FJsonValue>>& Array, const TSharedPtr<FJsonValue>& Find)
	{
		if (Find->Type == EJson::Object)
		{
			TSharedPtr<FJsonValue> FindKey = GetKeyField(Find);
			if (FindKey.IsValid())
			{
				const TSharedPtr<FJsonValue>* ExistingField = Array.FindByPredicate(
					[FindKey](const TSharedPtr<FJsonValue>& Value)
					{
						TSharedPtr<FJsonValue> ValueKey = GetKeyField(Value);
						if (ValueKey.IsValid())
						{
							return FJsonValue::CompareEqual(*FindKey.Get(), *ValueKey.Get());
						}
						return false;
					});

				if (ExistingField != nullptr)
				{
					return *ExistingField;
				}
			}
		}
		else
		{
			const TSharedPtr<FJsonValue>* ExistingField = Array.FindByPredicate(
				[Find](const TSharedPtr<FJsonValue>& Value)
				{
					return FJsonValue::CompareEqual(*Find.Get(), *Value.Get());
				});

			if (ExistingField != nullptr)
			{
				return *ExistingField;
			}
		}

		return TSharedPtr<FJsonValue>();
	}

	static bool RemoveValueFromArray(TArray<TSharedPtr<FJsonValue>>& Array, const TSharedPtr<FJsonValue>& Find)
	{
		if (Find->Type == EJson::Object)
		{
			TSharedPtr<FJsonValue> FindKey = GetKeyField(Find);
			if (FindKey.IsValid())
			{
				return Array.RemoveAll(
					[FindKey](const TSharedPtr<FJsonValue>& Value)
					{
						TSharedPtr<FJsonValue> ValueKey = GetKeyField(Value);
						if (ValueKey.IsValid())
						{
							return FJsonValue::CompareEqual(*FindKey.Get(), *ValueKey.Get());
						}
						return false;
					}) > 0;
			}
		}

		return Array.RemoveAll(
			[Find](const TSharedPtr<FJsonValue>& Value)
			{
				return FJsonValue::CompareEqual(*Find.Get(), *Value.Get());
			}) > 0;
	}

	static bool ApplyDeltaOperationsToArray(const TSharedPtr<FJsonObject>& OverrideObject, TArray<TSharedPtr<FJsonValue>>& DestArray)
	{
		const TArray<TSharedPtr<FJsonValue>>* OverrideSet = nullptr;
		if (OverrideObject->TryGetArrayField(TEXT("="), OverrideSet) && OverrideSet) // the latter part of this condition is just to satisfy static analysis
		{
			const TArray<TSharedPtr<FJsonValue>>& OverrideSetRef = *OverrideSet;

			DestArray.Reset();
			DestArray.Reserve(OverrideSetRef.Num());
			for (const TSharedPtr<FJsonValue>& SetValue : OverrideSetRef)
			{
				DestArray.Add(FJsonValue::Duplicate(SetValue));
			}
		}

		const TArray<TSharedPtr<FJsonValue>>* OverrideAdd = nullptr;
		if (OverrideObject->TryGetArrayField(TEXT("+"), OverrideAdd) && OverrideAdd)
		{
			const TArray<TSharedPtr<FJsonValue>>& OverrideAddRef = *OverrideAdd;

			DestArray.Reserve(DestArray.Num() + OverrideAddRef.Num());
			for (const TSharedPtr<FJsonValue>& AddValue : OverrideAddRef)
			{
				// check if this is a map with $key, $value fields
				if (AddValue->Type == EJson::Object)
				{
					TSharedPtr<FJsonValue> AddKey = GetKeyField(AddValue);
					if (AddKey.IsValid())
					{
						TSharedPtr<FJsonValue>* ExistingField = DestArray.FindByPredicate(
							[AddKey](const TSharedPtr<FJsonValue>& Value)
							{
								TSharedPtr<FJsonValue> ValueKey = GetKeyField(Value);
								if (ValueKey.IsValid())
								{
									return FJsonValue::CompareEqual(*ValueKey.Get(), *AddKey.Get());
								}
								return false;
							});

						if (ExistingField != nullptr)
						{
							// there was an existing field with the same key, override that value
							TSharedPtr<FJsonValue> AddValueField = GetValueField(AddValue);
							ensureAlways(AddValueField.IsValid());

							TSharedPtr<FJsonObject> ExistingObject = (*ExistingField)->AsObject();
							ExistingObject->SetField(TEXT("$value"), FJsonValue::Duplicate(AddValueField));
							continue;
						}
					}
				}

				DestArray.Add(FJsonValue::Duplicate(AddValue));
			}
		}

		const TArray<TSharedPtr<FJsonValue>>* OverrideRemove = nullptr;
		if (OverrideObject->TryGetArrayField(TEXT("-"), OverrideRemove) && OverrideRemove)
		{
			const TArray<TSharedPtr<FJsonValue>>& OverrideRemoveRef = *OverrideRemove;
			for (const TSharedPtr<FJsonValue>& RemoveValue : OverrideRemoveRef)
			{
				DestArray.RemoveAll(
					[RemoveValue](const TSharedPtr<FJsonValue>& Value)
					{
						TSharedPtr<FJsonValue> RemoveKey = GetKeyField(RemoveValue);
						TSharedPtr<FJsonValue> ExistingKey = GetKeyField(Value);
						if (RemoveKey.IsValid() && ExistingKey.IsValid())
						{
							// the existing and remove values have $key fields, compare those
							return FJsonValue::CompareEqual(*RemoveKey.Get(), *ExistingKey.Get());
						}

						if (ExistingKey.IsValid())
						{
							// the existing value has a $key field, compare it with the remove value, 
							// because we want to support the short-hand of "-": [ "Foo" ] instead of "-": [ { "$key": "Foo" } ]
							return FJsonValue::CompareEqual(*RemoveValue.Get(), *ExistingKey.Get());
						}

						// just compare values directly
						return FJsonValue::CompareEqual(*RemoveValue.Get(), *Value.Get());
					});
			}
		}
	
		if (OverrideSet != nullptr && 
			(OverrideAdd != nullptr || OverrideRemove != nullptr))
		{
			UE_LOGF(LogJsonConfig, Warning, "JSON container is malformed and contains both set and add/remove values.");
		}

		return OverrideSet != nullptr || OverrideAdd != nullptr || OverrideRemove != nullptr;
	}

	static bool ApplyOverridesToObject(const TSharedPtr<FJsonObject>& Overrides, TSharedPtr<FJsonObject>& Dest)
	{
		for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& Override : Overrides->Values)
		{
			TSharedPtr<FJsonValue> DestValue = Dest->TryGetField(Override.Key);
			if (!DestValue.IsValid())
			{
				const TSharedPtr<FJsonObject>* OverrideObject = nullptr;
				if (Override.Value->TryGetObject(OverrideObject) && OverrideObject)
				{
					TArray<TSharedPtr<FJsonValue>> DestArray;
					if (ApplyDeltaOperationsToArray(*OverrideObject, DestArray))
					{
						// was an array, DestArray is populated
						Dest->SetArrayField(FString(Override.Key), DestArray);
					}
					else
					{
						TSharedPtr<FJsonObject> DestObject = MakeShared<FJsonObject>();
						ApplyOverridesToObject(*OverrideObject, DestObject);
						Dest->SetObjectField(FString(Override.Key), DestObject);
					}
				}
				else
				{
					Dest->SetField(Override.Key, FJsonValue::Duplicate(Override.Value));
				}
			}
			else
			{
				switch (Override.Value->Type)
				{
					case EJson::Boolean:
					{
						bool OverrideBool, DestBool;
						if (Override.Value->TryGetBool(OverrideBool) && 
							DestValue->TryGetBool(DestBool))
						{
							Dest->SetBoolField(FString(Override.Key), OverrideBool);
						}
						continue;
					}
					case EJson::Number:
					{
						double OverrideNumber, DestNumber;
						if (Override.Value->TryGetNumber(OverrideNumber) &&
							DestValue->TryGetNumber(DestNumber))
						{
							Dest->SetNumberField(FString(Override.Key), OverrideNumber);
						}
						continue;
					}
					case EJson::String:
					{
						FString OverrideString, DestString;
						if (Override.Value->TryGetString(OverrideString) &&
							DestValue->TryGetString(DestString))
						{
							Dest->SetStringField(FString(Override.Key), OverrideString);
						}
						continue;
					}
					case EJson::Array:
					{
						if (Override.Value->Type == EJson::Array && 
							DestValue->Type == EJson::Array)
						{
							// our overrides has an array, so this is implicitly a set operation
							Dest->SetField(Override.Key, FJsonValue::Duplicate(Override.Value));
						}
						continue;
					}
					case EJson::Object:
					{
						const TSharedPtr<FJsonObject>* OverrideObject = nullptr;
						if (Override.Value->TryGetObject(OverrideObject))
						{
							check(OverrideObject);
							TArray<TSharedPtr<FJsonValue>>* DestArray = nullptr;
							TSharedPtr<FJsonObject>* DestObject = nullptr;
							if (DestValue->TryGetObject(DestObject) && DestObject)
							{
								ApplyOverridesToObject(*OverrideObject, *DestObject);
							}
							else if (DestValue->TryGetArray(DestArray) && DestArray)
							{
								ApplyDeltaOperationsToArray(*OverrideObject, *DestArray);
							}
						}
						continue;
					}
					default:
						ensureAlways(false);
						return false;
				}
			}
		}

		return true;
	}

	bool FJsonConfig::MergeThisWithParent()
	{
		MergedObject->Values.Reset();

		// pre-fill with parent's values if we have one
		if (ParentConfig.IsValid())
		{
			FJsonObject::Duplicate(ParentConfig->MergedObject, MergedObject);
		}

		// apply this object's overrides if we have any data
		if (OverrideObject.IsValid())
		{
			return ApplyOverridesToObject(OverrideObject, MergedObject);
		}

		return true;
	}

	bool FJsonConfig::TryGetBool(const FJsonPath& Path, bool& OutValue) const
	{
		TSharedPtr<FJsonValue> JsonValue;
		if (!TryGetJsonValue(Path, JsonValue))
		{
			return false;
		}
	
		return JsonValue->TryGetBool(OutValue);
	}

	bool FJsonConfig::TryGetString(const FJsonPath& Path, FString& OutValue) const
	{
		TSharedPtr<FJsonValue> JsonValue;
		if (!TryGetJsonValue(Path, JsonValue))
		{
			return false;
		}
	
		return JsonValue->TryGetString(OutValue);
	}

	bool FJsonConfig::TryGetString(const FJsonPath& Path, FName& OutValue) const
	{
		TSharedPtr<FJsonValue> JsonValue;
		if (!TryGetJsonValue(Path, JsonValue))
		{
			return false;
		}

		FString TempString;
		if (!JsonValue->TryGetString(TempString))
		{
			return false;
		}

		OutValue = *TempString;
		return true;
	}

	bool FJsonConfig::TryGetString(const FJsonPath& Path, FText& OutValue) const
	{
		TSharedPtr<FJsonValue> JsonValue;
		if (!TryGetJsonValue(Path, JsonValue))
		{
			return false;
		}
	
		FString TempString;
		if (!JsonValue->TryGetString(TempString))
		{
			return false;
		}

		OutValue = FText::FromString(TempString);
		return true;
	}

	bool FJsonConfig::TryGetArray(const FJsonPath& Path, TArray<bool>& OutArray) const
	{
		return TryGetArrayHelper(Path, OutArray, 
			[](const TSharedPtr<FJsonValue>& JsonValue, bool& OutBool)
			{
				return JsonValue->TryGetBool(OutBool);
			});
	}

	bool FJsonConfig::TryGetArray(const FJsonPath& Path, TArray<int8>& OutArray) const
	{
		return TryGetNumericArrayHelper(Path, OutArray);
	}

	bool FJsonConfig::TryGetArray(const FJsonPath& Path, TArray<int16>& OutArray) const
	{
		return TryGetNumericArrayHelper(Path, OutArray);
	}

	bool FJsonConfig::TryGetArray(const FJsonPath& Path, TArray<int32>& OutArray) const
	{
		return TryGetNumericArrayHelper(Path, OutArray);
	}

	bool FJsonConfig::TryGetArray(const FJsonPath& Path, TArray<int64>& OutArray) const
	{
		return TryGetNumericArrayHelper(Path, OutArray);
	}

	bool FJsonConfig::TryGetArray(const FJsonPath& Path, TArray<uint8>& OutArray) const
	{
		return TryGetNumericArrayHelper(Path, OutArray);
	}

	bool FJsonConfig::TryGetArray(const FJsonPath& Path, TArray<uint16>& OutArray) const
	{
		return TryGetNumericArrayHelper(Path, OutArray);
	}

	bool FJsonConfig::TryGetArray(const FJsonPath& Path, TArray<uint32>& OutArray) const
	{
		return TryGetNumericArrayHelper(Path, OutArray);
	}

	bool FJsonConfig::TryGetArray(const FJsonPath& Path, TArray<uint64>& OutArray) const
	{
		return TryGetNumericArrayHelper(Path, OutArray);
	}

	bool FJsonConfig::TryGetArray(const FJsonPath& Path, TArray<float>& OutArray) const
	{
		return TryGetNumericArrayHelper(Path, OutArray);
	}

	bool FJsonConfig::TryGetArray(const FJsonPath& Path, TArray<double>& OutArray) const
	{
		return TryGetNumericArrayHelper(Path, OutArray);
	}

	bool FJsonConfig::TryGetArray(const FJsonPath& Path, TArray<FString>& OutArray) const
	{
		return TryGetArrayHelper(Path, OutArray, 
			[](const TSharedPtr<FJsonValue>& JsonValue, FString& OutString)
			{
				return JsonValue->TryGetString(OutString);
			});
	}

	bool FJsonConfig::TryGetArray(const FJsonPath& Path, TArray<FText>& OutArray) const
	{
		return TryGetArrayHelper(Path, OutArray, 
			[](const TSharedPtr<FJsonValue>& JsonValue, FText& OutText)
			{
				FString StringValue;
				if (JsonValue->TryGetString(StringValue))
				{
					OutText = FText::FromString(StringValue);
					return true;
				}
				return false;
			});
	}

	bool FJsonConfig::TryGetArray(const FJsonPath& Path, TArray<FName>& OutArray) const
	{
		return TryGetArrayHelper(Path, OutArray, 
			[](const TSharedPtr<FJsonValue>& JsonValue, FName& OutName)
			{
				FString StringValue;
				if (JsonValue->TryGetString(StringValue))
				{
					OutName = *StringValue;
					return true;
				}
				return false;
			});
	}

	bool FJsonConfig::TryGetArray(const FJsonPath& Path, TArray<TSharedPtr<FJsonValue>>& OutValue) const
	{
		TSharedPtr<FJsonValue> JsonValue;
		if (!TryGetJsonValue(Path, JsonValue))
		{
			return false;
		}

		const TArray<TSharedPtr<FJsonValue>>* ValuePtr;
		if (!JsonValue->TryGetArray(ValuePtr))
		{
			return false;
		}

		OutValue = *ValuePtr;
		return true;
	}

	bool FJsonConfig::TryGetMap(const FJsonPath& Path, TArray<FJsonValuePair>& OutMap) const
	{
		TArray<TSharedPtr<FJsonValue>> Array;
		if (!TryGetArray(Path, Array))
		{
			return false;
		}

		OutMap.Reset();
		OutMap.Reserve(Array.Num());

		for (const TSharedPtr<FJsonValue>& Value : Array)
		{
			if (Value->Type != EJson::Object)
			{
				return false;
			}

			const TSharedPtr<FJsonObject>& Object = Value->AsObject();

			TSharedPtr<FJsonValue> KeyField = Object->TryGetField(TEXT("$key"));
			TSharedPtr<FJsonValue> ValueField = Object->TryGetField(TEXT("$value"));

			if (!KeyField.IsValid() || !ValueField.IsValid())
			{
				return false;
			}

			OutMap.Add(FJsonValuePair(KeyField, ValueField));
		}

		return true;
	}

	bool FJsonConfig::TryGetJsonObject(const FJsonPath& Path, TSharedPtr<FJsonObject>& OutValue) const
	{
		TSharedPtr<FJsonValue> JsonValue;
		if (!TryGetJsonValue(Path, JsonValue))
		{
			return false;
		}

		const TSharedPtr<FJsonObject>* ValuePtr;
		if (JsonValue->TryGetObject(ValuePtr))
		{
			OutValue = *ValuePtr;
			return true;
		}
		return false;
	}

	TSharedPtr<FJsonObject> FJsonConfig::GetRootObject() const
	{
		return MergedObject;
	}

	static TSharedPtr<FJsonValue> FindPartInObject(const TSharedPtr<FJsonObject>& Object, const FJsonPath::FPart& PathPart)
	{
		TSharedPtr<FJsonValue> NextValue = Object->GetFieldUntyped(FStringView(PathPart.Name));
		if (!NextValue.IsValid())
		{
			// part of path didn't exist
			return {};
		}

		if (PathPart.Index != INDEX_NONE)
		{
			const TArray<TSharedPtr<FJsonValue>>* ArrayPtr = nullptr;
			if (NextValue->TryGetArray(ArrayPtr))
			{
				if (!ArrayPtr->IsValidIndex(PathPart.Index))
				{
					// index out of range
					return {};
				}

				NextValue = (*ArrayPtr)[PathPart.Index];
			}
		}
		
		return NextValue;
	}

	bool FJsonConfig::TryGetJsonValue(const FJsonPath& Path, TSharedPtr<FJsonValue>& OutValue) const
	{
		if (!IsValid())
		{
			return false;
		}

		const TSharedPtr<FJsonObject>* CurrentObject = &MergedObject;

		for (int32 Idx = 0; Idx < Path.Length() - 1; ++Idx)
		{
			const TSharedPtr<FJsonValue> NextValue = FindPartInObject(*CurrentObject, Path[Idx]);
			if (!NextValue.IsValid() || !NextValue->TryGetObject(CurrentObject))
			{
				return false;
			}
		}

		const TSharedPtr<FJsonValue> LastValue = FindPartInObject(*CurrentObject, Path[Path.Length() - 1]);
		if (!LastValue.IsValid())
		{
			return false;
		}

		OutValue = LastValue;
		return true;
	}

	static bool SetValueHelper(TSharedPtr<FJsonValue>& CurrentValue, const TSharedPtr<FJsonValue>& NewValue, const TSharedPtr<FJsonValue>& ParentValue);

	static bool SetArrayValueHelper(const TSharedPtr<FJsonValue>& CurrentValue, const TArray<TSharedPtr<FJsonValue>>& NewArray, const TSharedPtr<FJsonValue>& ParentValue)
	{
		TArray<TSharedPtr<FJsonValue>>* CurrentArray = nullptr;
		if (CurrentValue->TryGetArray(CurrentArray))
		{
			// currently has array value so is implicitly a set, we can just overwrite it
			(*CurrentArray) = NewArray;
			return true;
		}

		TSharedPtr<FJsonObject>* CurrentObject = nullptr;
		if (!CurrentValue->TryGetObject(CurrentObject))
		{
			return false;
		}

		// we generally want to steer clear of set fields, but if one already exists then we'll just use it
		TSharedPtr<FJsonValue> SetField = (*CurrentObject)->TryGetField(TEXT("="));
		TArray<TSharedPtr<FJsonValue>>* SetArray = nullptr;
		if (SetField.IsValid() && SetField->TryGetArray(SetArray))
		{
			(*SetArray) = NewArray;

			// remove add and remove fields
			(*CurrentObject)->RemoveField(TEXT("+"));
			(*CurrentObject)->RemoveField(TEXT("-"));
			return true;
		}

		// build up a diff relative to the parent array
		const TArray<TSharedPtr<FJsonValue>>* ParentArray = nullptr;
		if (ParentValue.IsValid())
		{
			ParentValue->TryGetArray(ParentArray);
		}

		TArray<TSharedPtr<FJsonValue>> AddedValues;
		TArray<TSharedPtr<FJsonValue>> RemovedValues;

		if (ParentArray != nullptr)
		{
			// find all shared values
			TArray<TSharedPtr<FJsonValue>> SharedValues;
			SharedValues.Reserve(NewArray.Num());

			for (const TSharedPtr<FJsonValue>& NewElement : NewArray)
			{
				TSharedPtr<FJsonValue> ExistingElement = FindValueInArray(*ParentArray, NewElement);
				if (ExistingElement.IsValid())
				{
					SharedValues.Add(NewElement);
				}
			}

			// add all values in the new array that weren't shared to the added values
			AddedValues.Reserve(NewArray.Num() - SharedValues.Num());

			for (const TSharedPtr<FJsonValue>& NewElement : NewArray)
			{
				TSharedPtr<FJsonValue> ExistingElement = FindValueInArray(SharedValues, NewElement);
				if (!ExistingElement.IsValid())
				{
					AddedValues.Add(NewElement);
				}
			}
	
			// add all values in the parent array that weren't shared to the removed values
			RemovedValues.Reserve(ParentArray->Num() - SharedValues.Num());

			for (const TSharedPtr<FJsonValue>& ParentElement : *ParentArray)
			{
				TSharedPtr<FJsonValue> ExistingElement = FindValueInArray(SharedValues, ParentElement);
				if (!ExistingElement.IsValid())
				{
					RemovedValues.Add(ParentElement);
				}
			}
		}
		else
		{
			AddedValues = NewArray;
		}

		TSharedPtr<FJsonValue> AddField = (*CurrentObject)->TryGetField(TEXT("+"));
		if (AddField.IsValid())
		{
			TArray<TSharedPtr<FJsonValue>>* AddArray = nullptr;
			if (ensureAlwaysMsgf(AddField->TryGetArray(AddArray), TEXT("Invalid JSON config: \"+\" field in JSON config was not an array.")))
			{
				(*AddArray) = AddedValues;
			}
		}
		else if (AddedValues.Num() > 0)
		{
			(*CurrentObject)->SetArrayField(TEXT("+"), AddedValues);
		}

		TSharedPtr<FJsonValue> RemoveField = (*CurrentObject)->TryGetField(TEXT("-"));
		if (RemoveField.IsValid())
		{
			TArray<TSharedPtr<FJsonValue>>* RemoveArray = nullptr;
			if (ensureAlwaysMsgf(RemoveField->TryGetArray(RemoveArray), TEXT("Invalid JSON config: \"-\" field in JSON config was not an array.")))
			{
				(*RemoveArray) = RemovedValues;
			}
		}
		else if (RemovedValues.Num() > 0)
		{
			(*CurrentObject)->SetArrayField(TEXT("-"), RemovedValues);
		}

		return true;
	}

	static bool SetObjectValueHelper(const TSharedPtr<FJsonObject>& CurrentObject, const TSharedPtr<FJsonObject>& NewObject, const TSharedPtr<FJsonValue>& ParentValue)
	{
		TSharedPtr<FJsonObject>* ParentObject = nullptr;
		if (ParentValue.IsValid())
		{
			ParentValue->TryGetObject(ParentObject);
		}

		if (ParentObject == nullptr)
		{
			// no parent, just set everything we were given into the object
			for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& NewValuePair : NewObject->Values)
			{
				CurrentObject->SetField(NewValuePair.Key, NewValuePair.Value);
			}
		}
		else
		{
			for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& NewValuePair : NewObject->Values)
			{
				TSharedPtr<FJsonValue> ParentField = (*ParentObject)->TryGetField(NewValuePair.Key);
				if (!ParentField.IsValid())
				{
					// not set in parent, just set here
					CurrentObject->SetField(NewValuePair.Key, NewValuePair.Value);
				}
				else
				{
					// already set in parent, if we're identical we can just remove this field from this object
					if (FJsonValue::CompareEqual(*ParentField.Get(), *NewValuePair.Value.Get()))
					{
						CurrentObject->RemoveField(NewValuePair.Key);
					}
					else
					{
						TSharedPtr<FJsonValue> CurrentField = CurrentObject->TryGetField(NewValuePair.Key);
						if (!CurrentField.IsValid())
						{
							CurrentObject->SetField(NewValuePair.Key, NewValuePair.Value);
						}
						else
						{
							SetValueHelper(CurrentField, NewValuePair.Value, ParentField);
						}
					}
				}
			}
		}

		return true;
	}

	bool SetValueHelper(TSharedPtr<FJsonValue>& CurrentValue, const TSharedPtr<FJsonValue>& NewValue, const TSharedPtr<FJsonValue>& ParentValue)
	{
		check(CurrentValue.IsValid());
		check(NewValue.IsValid());

		EJson ValueType = NewValue->Type;

		switch (ValueType)
		{
			case EJson::Number:
			case EJson::Boolean:
			case EJson::String:
				if (CurrentValue->Type != ValueType)
				{
					return false;
				}

				CurrentValue = NewValue;
				return true;

			case EJson::Array:
				if (CurrentValue->Type != EJson::Object && CurrentValue->Type != EJson::Array)
				{
					return false;
				}

				return SetArrayValueHelper(CurrentValue, NewValue->AsArray(), ParentValue);

			case EJson::Object:
				if (CurrentValue->Type != EJson::Object)
				{
					return false;
				}

				return SetObjectValueHelper(CurrentValue->AsObject(), NewValue->AsObject(), ParentValue);
		}

		return false;
	}

	bool FJsonConfig::SetJsonValueInMerged(const FJsonPath& Path, const TSharedPtr<FJsonValue>& NewValue)
	{
		TSharedPtr<FJsonObject>* CurrentObject = &MergedObject;
	
		for (int32 Idx = 0; Idx < Path.Length() - 1; ++Idx)
		{
			const FJsonPath::FPart& CurrentPart = Path[Idx];
			const TSharedPtr<FJsonValue> NextValue = FindPartInObject(*CurrentObject, CurrentPart);
			if (!NextValue.IsValid() || 
				!NextValue->TryGetObject(CurrentObject))
			{
				if (CurrentPart.Index == INDEX_NONE)
				{
					// create an empty object
					(*CurrentObject)->SetObjectField(Path[Idx].Name, MakeShared<FJsonObject>());
				}
				else
				{
					// array entry that doesn't exist
					return false;
				}
			}
		}

		const FJsonPath::FPart& LastPart = Path[Path.Length() - 1];

		TSharedPtr<FJsonValue> LastValue = FindPartInObject(*CurrentObject, LastPart);
		if (!LastValue.IsValid())
		{
			if (LastPart.Index == INDEX_NONE)
			{
				// value doesn't exist and this isn't an array entry, should just set it
				(*CurrentObject)->SetField(LastPart.Name, NewValue);
				return true;
			}
			else
			{
				// array entry that doesn't exist
				return false;
			}
		}

		return SetValueHelper(LastValue, NewValue, TSharedPtr<FJsonValue>());
	}

	bool FJsonConfig::SetJsonValueInOverride(const FJsonPath& Path, const TSharedPtr<FJsonValue>& NewValue, const TSharedPtr<FJsonValue>& PreviousValue, const TSharedPtr<FJsonValue>& ParentValue)
	{
		TSharedPtr<FJsonObject>* CurrentObject = &OverrideObject;
	
		for (int32 Idx = 0; Idx < Path.Length() - 1; ++Idx)
		{
			const FJsonPath::FPart& CurrentPart = Path[Idx];
			const TSharedPtr<FJsonValue> NextValue = FindPartInObject(*CurrentObject, CurrentPart);
			if (!NextValue.IsValid())
			{
				// since we know this isn't the last part, we can safely add an object here
				(*CurrentObject)->SetObjectField(CurrentPart.Name, MakeShared<FJsonObject>());

				TSharedPtr<FJsonValue> CreatedValue = (*CurrentObject)->TryGetField(CurrentPart.Name);
				CreatedValue->TryGetObject(CurrentObject);
			}
			else if (!NextValue->TryGetObject(CurrentObject))
			{
				FString SubPath = Path.GetSubPath(Idx + 1).ToString();
				ensureMsgf(false, TEXT("JSON value at \"%s\" not a JSON object in path: %s"), *SubPath, *Path.ToString());
			}

			if (CurrentPart.Index != INDEX_NONE)
			{
				// if an index is set, there's an array in the path, eg. Foo[1].Bar
				ensureAlwaysMsgf(false, TEXT("The array in the middle of a path needs to be figured out..."));
			}
		}

		const FJsonPath::FPart& LastPart = Path[Path.Length() - 1];

		TSharedPtr<FJsonValue> LastValue = FindPartInObject(*CurrentObject, LastPart);
		if (!LastValue.IsValid())
		{
			// field doesn't exist 

			if (LastPart.Index == INDEX_NONE)
			{
				// this isn't an array entry, just set it now
				if (NewValue->Type == EJson::Object || NewValue->Type == EJson::Array)
				{
					// create a new empty object that we can diff against
					TSharedPtr<FJsonValue> CreatedValue = MakeShared<FJsonValueObject>(MakeShared<FJsonObject>());
					(*CurrentObject)->SetField(LastPart.Name, CreatedValue);
					SetValueHelper(CreatedValue, NewValue, ParentValue);
				}
				else
				{
					(*CurrentObject)->SetField(LastPart.Name, NewValue);
				}

				return true;
			}
			else if (!PreviousValue.IsValid())
			{
				// out of bounds
				return false;
			}
			else
			{
				// valid index, but no array field created in OverrideObject yet, create one
				(*CurrentObject)->SetObjectField(LastPart.Name, MakeShared<FJsonObject>());
			}
		}

		if (LastPart.Index != INDEX_NONE)
		{
			// last part is an array, eg. Foo.Bar[0] - array should absolutely be created by now
			TSharedPtr<FJsonValue> ArrayField = (*CurrentObject)->TryGetField(LastPart.Name);
			check(ArrayField.IsValid());

			TArray<TSharedPtr<FJsonValue>>* ArrayValue = nullptr;
			if (ArrayField->TryGetArray(ArrayValue))
			{
				// override is an array, so implicitly a set

				if (ArrayValue->IsValidIndex(LastPart.Index))
				{
					(*ArrayValue)[LastPart.Index] = NewValue;
					return true;
				}

				// index out of bounds
				return false;
			}

			TSharedPtr<FJsonObject>* ArrayObject = nullptr;
			if (ArrayField->TryGetObject(ArrayObject))
			{
				// the field is an object, so we need to modify existing override fields

				// check if we have a set field
				TSharedPtr<FJsonValue> SetField = (*ArrayObject)->TryGetField(TEXT("="));
				TArray<TSharedPtr<FJsonValue>>* SetContents = nullptr;
				if (SetField.IsValid() && SetField->TryGetArray(SetContents))
				{
					if (SetContents->IsValidIndex(LastPart.Index))
					{
						(*SetContents)[LastPart.Index] = NewValue;
						return true;
					}

					// index out of bounds
					return false;
				}

				// see if the previous value has already been removed in an override
				check(PreviousValue.IsValid());

				TSharedPtr<FJsonValue> RemoveField = (*ArrayObject)->TryGetField(TEXT("-"));
				TArray<TSharedPtr<FJsonValue>>* RemoveContents = nullptr;
				if (RemoveField.IsValid() && RemoveField->TryGetArray(RemoveContents))
				{
					TSharedPtr<FJsonValue> ExistingRemove = FindValueInArray(*RemoveContents, PreviousValue);
					if (!ExistingRemove.IsValid())
					{
						// hasn't already been removed, remove it now
						RemoveContents->Add(PreviousValue);
					}
				
					// remove the new value from the remove list, otherwise we can end up in a situation where a value is added _and_ removed
					RemoveValueFromArray(*RemoveContents, NewValue);
				}
				else
				{
					// add a remove field with the previous value
					TArray<TSharedPtr<FJsonValue>> NewRemoveContents; 
					NewRemoveContents.Add(PreviousValue);

					(*ArrayObject)->SetArrayField(TEXT("-"), NewRemoveContents);
				}

				// see if the new value has already been added in an override
				TSharedPtr<FJsonValue> AddField = (*ArrayObject)->TryGetField(TEXT("+"));
				TArray<TSharedPtr<FJsonValue>>* AddContents = nullptr;
				if (AddField.IsValid() && AddField->TryGetArray(AddContents))
				{
					TSharedPtr<FJsonValue> ExistingAdd = FindValueInArray(*AddContents, NewValue);
					if (!ExistingAdd.IsValid())
					{
						// hasn't already been added, add it now
						AddContents->Add(NewValue);
					}

					// remove the old value from the add list, otherwise we can end up in a situation where a value is added _and_ removed
					RemoveValueFromArray(*AddContents, PreviousValue);
				}
				else
				{
					// add an add field with the new value
					TArray<TSharedPtr<FJsonValue>> NewAddContents; 
					NewAddContents.Add(NewValue);

					(*ArrayObject)->SetArrayField(TEXT("+"), NewAddContents);
				}
			}

			return true;
		}
	
		check(LastValue);
		return SetValueHelper(LastValue, NewValue, ParentValue);
	}

	bool FJsonConfig::RemoveJsonValueFromOverride(const FJsonPath& Path, const TSharedPtr<FJsonValue>& CurrentValue)
	{
		TSharedPtr<FJsonObject>* CurrentObject = &OverrideObject;
	
		for (int32 Idx = 0; Idx < Path.Length() - 1; ++Idx)
		{
			const TSharedPtr<FJsonValue> NextValue = FindPartInObject(*CurrentObject, Path[Idx]);
			if (!NextValue.IsValid() || !NextValue->TryGetObject(CurrentObject))
			{
				return false;
			}
		}

		const FJsonPath::FPart& LastPart = Path[Path.Length() - 1];

		if (LastPart.Index != INDEX_NONE)
		{
			TSharedPtr<FJsonValue> ArrayField = (*CurrentObject)->TryGetField(LastPart.Name);
			if (!ArrayField.IsValid())
			{
				return false;
			}

			TArray<TSharedPtr<FJsonValue>>* ArrayPtr = nullptr;
			if (ArrayField->TryGetArray(ArrayPtr))
			{
				// array field is an array, so implicitly a set
				RemoveValueFromArray(*ArrayPtr, CurrentValue);
				return true;
			}

			TSharedPtr<FJsonObject>* ArrayObject = nullptr;
			if (ArrayField->TryGetObject(ArrayObject))
			{
				{
					TSharedPtr<FJsonValue> SetField = (*ArrayObject)->TryGetField(TEXT("="));
					TArray<TSharedPtr<FJsonValue>>* SetContents = nullptr;
					if (SetField.IsValid() && SetField->TryGetArray(SetContents))
					{
						RemoveValueFromArray(*SetContents, CurrentValue);
						return true;
					}
				}

				{
					TSharedPtr<FJsonValue> AddField = (*ArrayObject)->TryGetField(TEXT("+"));
					TArray<TSharedPtr<FJsonValue>>* AddContents = nullptr;
					if (AddField.IsValid() && AddField->TryGetArray(AddContents))
					{
						RemoveValueFromArray(*AddContents, CurrentValue);

						// remove the "+" array if it's now empty, since it's unnecessary
						if (AddContents->Num() == 0)
						{
							(*ArrayObject)->RemoveField(TEXT("+"));
						}
					}
				}

				// if there's a value set in the parent, then it might have been removed in this config, so we need to remove that
				TSharedPtr<FJsonValue> ParentValue;
				if (ParentConfig.IsValid())
				{
					if (ParentConfig->TryGetJsonValue(Path, ParentValue))
					{
						TSharedPtr<FJsonValue> RemoveField = (*ArrayObject)->TryGetField(TEXT("-"));
						TArray<TSharedPtr<FJsonValue>>* RemoveContents = nullptr;
						if (RemoveField.IsValid() && RemoveField->TryGetArray(RemoveContents))
						{
							RemoveValueFromArray(*RemoveContents, ParentValue);

							// remove the "-" array if it's now empty, since it's unnecessary
							if (RemoveContents->Num() == 0)
							{
								(*ArrayObject)->RemoveField(TEXT("-"));
							}
						}
					}
				}
			}

			return true;
		}

		(*CurrentObject)->RemoveField(LastPart.Name);

		return true;
	}

	static bool ShouldAlwaysKeep(const FJsonPath& Path)
	{
		static const TCHAR* AlwaysKeep[] =
		{
			TEXT("$type")
		};

		if (Path.Length() > 0)
		{
			const FJsonPath::FPart& LastPart = Path[Path.Length() - 1];
			for (const TCHAR* Key : AlwaysKeep)
			{
				if (LastPart.Name == Key)
				{
					return true;
				}
			}
		}

		return false;
	}

	bool FJsonConfig::SetJsonValue(const FJsonPath& Path, const TSharedPtr<FJsonValue>& NewValue)
	{
		TSharedPtr<FJsonValue> PreviousValue;
		TryGetJsonValue(Path, PreviousValue);

		TSharedPtr<FJsonValue> MergedValue = FJsonValue::Duplicate(NewValue);
		if (!SetJsonValueInMerged(Path, MergedValue))
		{
			return false;
		}

		TSharedPtr<FJsonValue> ParentValue;

		// this was a valid change in the merged object 
		// we need to either add or remove it from this object, depending on if it differs from our parent
		bool bShouldRemove = false;
		if (!ShouldAlwaysKeep(Path))
		{
			if (ParentConfig.IsValid())
			{
				if (ParentConfig->TryGetJsonValue(Path, ParentValue))
				{
					if (FJsonValue::CompareEqual(*ParentValue.Get(), *NewValue.Get()))
					{
						// same as inherited, remove it from this
						bShouldRemove = true;
					}
				}
			}
		}

		if (bShouldRemove)
		{
			return RemoveJsonValueFromOverride(Path, PreviousValue);
		}

		TSharedPtr<FJsonValue> OverrideValue = FJsonValue::Duplicate(NewValue);
		if (!SetJsonValueInOverride(Path, OverrideValue, PreviousValue, ParentValue))
		{
			return false;
		}

		OnConfigChanged.ExecuteIfBound();
		return true;
	}

	bool FJsonConfig::SetString(const FJsonPath& Path, const FText& Value)
	{
		TSharedPtr<FJsonValue> JsonValue = MakeShared<FJsonValueString>(Value.ToString());
		return SetJsonValue(Path, JsonValue);
	}

	bool FJsonConfig::SetString(const FJsonPath& Path, FStringView Value)
	{
		TSharedPtr<FJsonValue> JsonValue = MakeShared<FJsonValueString>(FString(Value));
		return SetJsonValue(Path, JsonValue);
	}

	bool FJsonConfig::SetBool(const FJsonPath& Path, bool Value)
	{
		TSharedPtr<FJsonValue> JsonValue = MakeShared<FJsonValueBoolean>(Value);
		return SetJsonValue(Path, JsonValue);
	}

	bool FJsonConfig::SetJsonObject(const FJsonPath& Path, const TSharedPtr<FJsonObject>& Object)
	{
		TSharedPtr<FJsonValue> JsonValue = MakeShared<FJsonValueObject>(Object);
		return SetJsonValue(Path, JsonValue);
	}

	bool FJsonConfig::SetJsonArray(const FJsonPath& Path, const TArray<TSharedPtr<FJsonValue>>& Array)
	{
		TSharedPtr<FJsonValue> JsonValue = MakeShared<FJsonValueArray>(Array);
		return SetJsonValue(Path, JsonValue);
	}

	bool FJsonConfig::HasOverride(const FJsonPath& Path) const
	{
		if (!IsValid())
		{
			return false;
		}

		if (!Path.IsValid())
		{
			return false;
		}

		const TSharedPtr<FJsonObject>* CurrentObject = &OverrideObject;

		for (int32 Idx = 0; Idx < Path.Length() - 1; ++Idx)
		{
			const TSharedPtr<FJsonValue> NextValue = FindPartInObject(*CurrentObject, Path[Idx]);
			if (!NextValue.IsValid() || !NextValue->TryGetObject(CurrentObject))
			{
				return false;
			}
		}

		const TSharedPtr<FJsonValue> LastValue = FindPartInObject(*CurrentObject, Path[Path.Length() - 1]);
		if (!LastValue.IsValid())
		{
			return false;
		}

		return true;
	}

	bool FJsonConfig::SetRootObject(const TSharedPtr<FJsonObject>& Object)
	{
		if (!Object.IsValid())
		{
			return false;
		}

		TArray<FJsonObject::FStringType> NewKeys;
		Object->Values.GenerateKeyArray(NewKeys);

		TArray<FJsonObject::FStringType> ExistingKeys;
		OverrideObject->Values.GenerateKeyArray(ExistingKeys);

		for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& Pair : Object->Values)
		{
			FJsonPath Path(Pair.Key);
			SetJsonValue(Path, Pair.Value);
		}

		// remove all keys that are not in the new object
		for (const FJsonObject::FStringType& Key : ExistingKeys)
		{
			if (!NewKeys.Contains(Key))
			{
				OverrideObject->RemoveField(FStringView(Key));
			}
		}

		MergeThisWithParent();
		OnConfigChanged.ExecuteIfBound();

		return true;
	}
}
