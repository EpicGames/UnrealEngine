// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorConfigTestHelpers.h"

#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Templates/SharedPointer.h"

namespace FEditorConfigTestHelpers
{
	bool AreJsonStringsEquivalent(const FString& Actual, const FString& Expected)
	{
		TSharedPtr<FJsonObject> ActualObject = MakeShared<FJsonObject>();
		TSharedRef<FJsonStringReader> ActualReader = FJsonStringReader::Create(Actual);
		if (!FJsonSerializer::Deserialize(ActualReader.Get(), ActualObject))
		{
			return false;
		}

		TSharedPtr<FJsonObject> ExpectedObject = MakeShared<FJsonObject>();
		TSharedRef<FJsonStringReader> ExpectedReader = FJsonStringReader::Create(Expected);
		if (!FJsonSerializer::Deserialize(ExpectedReader.Get(), ExpectedObject))
		{
			return false;
		}

		return FJsonValue::CompareEqual(FJsonValueObject(ActualObject), FJsonValueObject(ExpectedObject));
	}
}