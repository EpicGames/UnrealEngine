// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"

namespace FEditorConfigTestHelpers
{
	// check that two serialized JSON objects are equivalent
	bool AreJsonStringsEquivalent(const FString& Actual, const FString& Expected);
}