// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS 

#include "EditorConfigTests.h"
#include "EditorConfig.h"
#include "EditorConfigTestHelpers.h"
#include "Engine/StaticMesh.h"
#include "Misc/AutomationTest.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorConfigTests)

bool FEditorConfigTestSimpleStruct::operator==(const FEditorConfigTestSimpleStruct& Other) const
{
	return Bool == Other.Bool &&
		Int == Other.Int &&
		String == Other.String &&
		Float == Other.Float &&
		Array == Other.Array;
}

bool FEditorConfigTestSimpleStruct::operator!=(const FEditorConfigTestSimpleStruct& Other) const
{
	return !(*this == Other);
}

bool FEditorConfigTestKey::operator==(const FEditorConfigTestKey& Other) const
{
	return Name == Other.Name &&
		Number == Other.Number;
}

bool FEditorConfigTestKey::operator!=(const FEditorConfigTestKey& Other) const
{
	return !(*this == Other);
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditorConfigTests_Save_EnumStruct, "EditorConfig.Save.EnumStruct", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FEditorConfigTests_Save_EnumStruct::RunTest(const FString& Parameters)
{
	FEditorConfigTestEnumStruct Actual;
	Actual.Before = -1;
	Actual.Enum = EEditorConfigTestEnum::One;
	Actual.After = -1;

	FEditorConfig Config;
	Config.SetRootStruct(Actual, FEditorConfig::EPropertyFilter::All);

	FString ActualString;
	if (!Config.SaveToString(ActualString))
	{
		AddError("Failed to save to string.");
		return false;
	}

	const FString Expected =
R"_JSON({ 
	"$type": "EditorConfigTestEnumStruct",
	"Before": -1,
	"Enum": "EEditorConfigTestEnum::One",
	"After": -1
})_JSON";

	if (!FEditorConfigTestHelpers::AreJsonStringsEquivalent(ActualString, Expected))
	{
		AddError(FString::Printf(TEXT("Contents does not match. Expected:\n%s\n\nActual:\n%s"), *Expected, *ActualString));
		return false;
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditorConfigTests_Load_EnumStruct, "EditorConfig.Load.EnumStruct", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FEditorConfigTests_Load_EnumStruct::RunTest(const FString& Parameters)
{
	// there was a bug where enums were reading past the underlying property and returning the wrong value as a result
	// this test is meant to catch that

	const FString Contents =
R"_JSON({ 
	"$type": "EditorConfigTestEnumStruct",
	"Before": -1,
	"Enum": "EEditorConfigTestEnum::One",
	"After": -1
})_JSON";

	FEditorConfig Config;
	if (!Config.LoadFromString(Contents))
	{
		AddError("Failed to load from string.");
		return false;
	}

	FEditorConfigTestEnumStruct Struct;
	if (!Config.TryGetRootStruct(Struct, FEditorConfig::EPropertyFilter::All))
	{
		AddError("Failed to load into struct.");
		return false;
	}

	FEditorConfigTestEnumStruct Comparison;
	Comparison.Before = -1;
	Comparison.Enum = EEditorConfigTestEnum::One;
	Comparison.After = -1;

	bool bResult = true;
	bResult &= TestEqual(TEXT("SimpleStruct"), Struct.Before, Comparison.Before);
	bResult &= TestEqual(TEXT("SimpleStruct"), Struct.Enum, Comparison.Enum);
	bResult &= TestEqual(TEXT("SimpleStruct"), Struct.After, Comparison.After);
	return bResult;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditorConfigTests_Load_SimpleStruct_Empty, "EditorConfig.Load.SimpleStruct.Empty", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FEditorConfigTests_Load_SimpleStruct_Empty::RunTest(const FString& Parameters)
{
	const FString Contents =
R"_JSON({ 
	"$type": "EditorConfigTestSimpleStruct"
})_JSON";

	FEditorConfig Config;
	if (!Config.LoadFromString(Contents))
	{
		AddError("Failed to load from string.");
		return false;
	}

	FEditorConfigTestSimpleStruct Struct;
	if (!Config.TryGetRootStruct(Struct, FEditorConfig::EPropertyFilter::All))
	{
		AddError("Failed to load into struct.");
		return false;
	}

	FEditorConfigTestSimpleStruct Comparison;

	bool bResult = true;
	bResult &= TestEqual(TEXT("SimpleStruct"), Struct, Comparison);
	return bResult;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditorConfigTests_Load_SimpleStruct_All, "EditorConfig.Load.SimpleStruct.All", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FEditorConfigTests_Load_SimpleStruct_All::RunTest(const FString& Parameters)
{
	const FString Contents =
R"_JSON({ 
	"$type": "EditorConfigTestSimpleStruct",
	"Bool": false,
	"Int": 42,
	"String": "foo",
	"Float": 21.0,
	"Array": [ "foo", "bar" ]
})_JSON";

	FEditorConfig Config;
	if (!Config.LoadFromString(Contents))
	{
		AddError("Failed to load from string.");
		return false;
	}

	FEditorConfigTestSimpleStruct Struct;
	if (!Config.TryGetRootStruct(Struct, FEditorConfig::EPropertyFilter::All))
	{
		AddError("Failed to load into struct.");
		return false;
	}
	
	bool bResult = true;
	bResult &= TestEqual(TEXT("Bool"), Struct.Bool, false);
	bResult &= TestEqual(TEXT("Int"), Struct.Int, 42);
	bResult &= TestEqual(TEXT("String"), Struct.String, TEXT("foo"));
	bResult &= TestEqual(TEXT("Float"), Struct.Float, 21.0f);
	bResult &= TestEqual(TEXT("Array"), Struct.Array, TArray<FString> { "foo", "bar"});
	return bResult;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditorConfigTests_Load_SimpleStruct_Partial, "EditorConfig.Load.SimpleStruct.All", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FEditorConfigTests_Load_SimpleStruct_Partial::RunTest(const FString& Parameters)
{
	const FString Contents =
R"_JSON({ 
	"$type": "EditorConfigTestSimpleStruct",
	"String": "foo",
	"Array": [ "foo", "bar" ]
})_JSON";

	FEditorConfig Config;
	if (!Config.LoadFromString(Contents))
	{
		AddError("Failed to load from string.");
		return false;
	}

	FEditorConfigTestSimpleStruct Struct;
	if (!Config.TryGetRootStruct(Struct, FEditorConfig::EPropertyFilter::All))
	{
		AddError("Failed to load into struct.");
		return false;
	}

	bool bResult = true;
	bResult &= TestEqual(TEXT("Bool"), Struct.Bool, true);
	bResult &= TestEqual(TEXT("Int"), Struct.Int, 5);
	bResult &= TestEqual(TEXT("String"), Struct.String, TEXT("foo"));
	bResult &= TestEqual(TEXT("Float"), Struct.Float, 5.0f);
	bResult &= TestEqual(TEXT("Array"), Struct.Array, TArray<FString> { "foo", "bar"});
	return bResult;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditorConfigTests_Load_SimpleStruct_Filtered, "EditorConfig.Load.SimpleStruct.All", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FEditorConfigTests_Load_SimpleStruct_Filtered::RunTest(const FString& Parameters)
{
	const FString Contents =
R"_JSON({ 
	"$type": "EditorConfigTestSimpleStruct",
	"Bool": false,
	"Int": 42,
	"Float": 42,
	"String": "foo",
	"Array": [ "foo", "bar" ]
})_JSON";

	FEditorConfig Config;
	if (!Config.LoadFromString(Contents))
	{
		AddError("Failed to load from string.");
		return false;
	}

	FEditorConfigTestSimpleStruct Struct;
	if (!Config.TryGetRootStruct(Struct, FEditorConfig::EPropertyFilter::MetadataOnly))
	{
		AddError("Failed to load into struct.");
		return false;
	}

	bool bResult = true;
	bResult &= TestEqual(TEXT("Bool"), Struct.Bool, true);
	bResult &= TestEqual(TEXT("Int"), Struct.Int, 5);
	bResult &= TestEqual(TEXT("String"), Struct.String, TEXT("foo"));
	bResult &= TestEqual(TEXT("Float"), Struct.Float, 5.0f);
	bResult &= TestEqual(TEXT("Array"), Struct.Array, TArray<FString> { "foo", "bar"});
	return bResult;
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditorConfigTests_Load_ComplexArrayStruct, "EditorConfig.Load.ComplexArray", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FEditorConfigTests_Load_ComplexArrayStruct::RunTest(const FString& Parameters)
{
	const FString Contents =
R"_JSON({ 
	"$type": "EditorConfigTestComplexArray",
	"Array":
	[ 
		{
			"$type": "EditorConfigTestKey",
			"Name": "foo", 
			"Number": 42.0
		},
		{
			"$type": "EditorConfigTestKey"
		}
	]
})_JSON";

	FEditorConfig Config;
	if (!Config.LoadFromString(Contents))
	{
		AddError("Failed to load from string.");
		return false;
	}

	FEditorConfigTestComplexArray Struct;
	if (!Config.TryGetRootStruct(Struct, FEditorConfig::EPropertyFilter::All))
	{
		AddError("Failed to load into struct.");
		return false;
	}

	bool bResult = true;
	bResult &= TestEqual(TEXT("Count"), Struct.Array.Num(), 2);
	bResult &= TestEqual(TEXT("First.Name"), Struct.Array[0].Name, TEXT("foo"));
	bResult &= TestEqual(TEXT("First.Number"), Struct.Array[0].Number, 42.0);
	bResult &= TestEqual(TEXT("Second.Name"), Struct.Array[1].Name, TEXT(""));
	bResult &= TestEqual(TEXT("Second.Number"), Struct.Array[1].Number, 5.0);
	return bResult;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditorConfigTests_Load_SimpleSet, "EditorConfig.Load.SimpleSet", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FEditorConfigTests_Load_SimpleSet::RunTest(const FString& Parameters)
{
	const FString Contents =
R"_JSON({ 
	"$type": "EditorConfigTestSimpleSet",
	"Set": [ "foo", "bar", "foo" ]
})_JSON";

	FEditorConfig Config;
	if (!Config.LoadFromString(Contents))
	{
		AddError("Failed to load from string.");
		return false;
	}

	FEditorConfigTestSimpleSet Struct;
	if (!Config.TryGetRootStruct(Struct, FEditorConfig::EPropertyFilter::All))
	{
		AddError("Failed to load into struct.");
		return false;
	}

	bool bResult = true;
	bResult &= TestEqual(TEXT("Count"), Struct.Set.Num(), 2);
	bResult &= TestTrue(TEXT("First"), Struct.Set.Contains("foo"));
	bResult &= TestTrue(TEXT("Second"), Struct.Set.Contains("bar"));
	return bResult;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditorConfigTests_Load_ComplexSet, "EditorConfig.Load.ComplexSet", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FEditorConfigTests_Load_ComplexSet::RunTest(const FString& Parameters)
{
	const FString Contents =
R"_JSON({ 
	"$type": "EditorConfigTestComplexSet",
	"Set":
	[ 
		{
			"$type": "EditorConfigTestKey",
			"Name": "foo", 
			"Number": 42.0
		},
		{
			"$type": "EditorConfigTestKey"
		}
	]
})_JSON";

	FEditorConfig Config;
	if (!Config.LoadFromString(Contents))
	{
		AddError("Failed to load from string.");
		return false;
	}

	FEditorConfigTestComplexSet Struct;
	if (!Config.TryGetRootStruct(Struct, FEditorConfig::EPropertyFilter::All))
	{
		AddError("Failed to load into struct.");
		return false;
	}

	bool bResult = true;
	bResult &= TestEqual(TEXT("Count"), Struct.Set.Num(), 2);

	int32 Index = 0;
	for (const FEditorConfigTestKey& Key : Struct.Set)
	{
		if (Index == 0)
		{
			bResult &= TestEqual(TEXT("First.Name"), Key.Name, TEXT("foo"));
			bResult &= TestEqual(TEXT("First.Number"), Key.Number, 42.0);
		}
		else if (Index == 1)
		{
			bResult &= TestEqual(TEXT("Second.Name"), Key.Name, TEXT(""));
			bResult &= TestEqual(TEXT("Second.Number"), Key.Number, 5.0);
		}

		++Index;
	}
	
	return bResult;
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditorConfigTests_Load_SimpleMap, "EditorConfig.Load.SimpleMap", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FEditorConfigTests_Load_SimpleMap::RunTest(const FString& Parameters)
{
	const FString Contents =
R"_JSON({ 
	"$type": "EditorConfigTestSimpleMap",
	"Map":
	{
		"foo": "one",
		"bar": "two"	
	}
})_JSON";

	FEditorConfig Config;
	if (!Config.LoadFromString(Contents))
	{
		AddError("Failed to load from string.");
		return false;
	}

	FEditorConfigTestSimpleMap Struct;
	if (!Config.TryGetRootStruct(Struct, FEditorConfig::EPropertyFilter::All))
	{
		AddError("Failed to load into struct.");
		return false;
	}

	bool bResult = true;
	bResult &= TestEqual(TEXT("Count"), Struct.Map.Num(), 2);
	bResult &= TestEqual(TEXT("First"), Struct.Map[TEXT("foo")], FString(TEXT("one")));
	bResult &= TestEqual(TEXT("Second"), Struct.Map[TEXT("bar")], FString(TEXT("two")));
	return bResult;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditorConfigTests_Load_ComplexMap, "EditorConfig.Load.ComplexMap", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FEditorConfigTests_Load_ComplexMap::RunTest(const FString& Parameters)
{
	const FString Contents =
R"_JSON({ 
	"$type": "EditorConfigTestComplexMap",
	"Map":
	[
		{
			"$key":
			{
				"$type": "EditorConfigTestKey",
				"Name": "foo", 
				"Number": 42.0
			},
			"$value":
			{
				"$type": "EditorConfigTestKey",
				"Name": "one", 
				"Number": 1
			}
		},
		{
			"$key":
			{
				"$type": "EditorConfigTestKey",
				"Name": "bar"
			},
			"$value":
			{
				"$type": "EditorConfigTestKey",
				"Number": 2
			}
		}
	]
})_JSON";

	FEditorConfig Config;
	if (!Config.LoadFromString(Contents))
	{
		AddError("Failed to load from string.");
		return false;
	}

	FEditorConfigTestComplexMap Struct;
	if (!Config.TryGetRootStruct(Struct, FEditorConfig::EPropertyFilter::All))
	{
		AddError("Failed to load into struct.");
		return false;
	}

	bool bResult = true;
	bResult &= TestEqual(TEXT("Count"), Struct.Map.Num(), 2);

	int32 Index = 0;
	for (const TPair<FEditorConfigTestKey, FEditorConfigTestKey>& Pair : Struct.Map)
	{
		if (Index == 0)
		{
			bResult &= TestEqual(TEXT("First.Key.Name"), Pair.Key.Name, TEXT("foo"));
			bResult &= TestEqual(TEXT("First.Key.Number"), Pair.Key.Number, 42.0);

			bResult &= TestEqual(TEXT("First.Value.Name"), Pair.Value.Name, TEXT("one"));
			bResult &= TestEqual(TEXT("First.Value.Number"), Pair.Value.Number, 1.0);
		}
		else if (Index == 1)
		{
			bResult &= TestEqual(TEXT("Second.Key.Name"), Pair.Key.Name, TEXT("bar"));
			bResult &= TestEqual(TEXT("Second.Key.Number"), Pair.Key.Number, 5.0);

			bResult &= TestEqual(TEXT("Second.Value.Name"), Pair.Value.Name, TEXT(""));
			bResult &= TestEqual(TEXT("Second.Value.Number"), Pair.Value.Number, 2.0);
		}

		++Index;
	}

	return bResult;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditorConfigTests_Load_MapWithOverride, "EditorConfig.Load.MapWithOverride", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FEditorConfigTests_Load_MapWithOverride::RunTest(const FString& Parameters)
{
	const FString ParentContents =
R"_JSON({ 
	"$type": "EditorConfigTestSimpleKeyComplexValueMap",
	"Map":
	{
		"foo":
		{
			"$type": "EditorConfigTestKey",
			"Name": "one", 
			"Number": 1
		}
	}
})_JSON";

	const FString ChildContents =
R"_JSON({ 
	"$type": "EditorConfigTestSimpleKeyComplexValueMap",
	"Map":
	{
		"foo":
		{
			"Number": 42
		}
	}
})_JSON";

	TSharedRef<FEditorConfig> ParentConfig = MakeShared<FEditorConfig>();
	if (!ParentConfig->LoadFromString(ParentContents))
	{
		AddError("Failed to load from string.");
		return false;
	}

	FEditorConfig ChildConfig;
	if (!ChildConfig.LoadFromString(ChildContents))
	{
		AddError("Failed to load from string.");
		return false;
	}

	ChildConfig.SetParent(ParentConfig);

	FEditorConfigTestSimpleKeyComplexValueMap Struct;
	if (!ChildConfig.TryGetRootStruct(Struct, FEditorConfig::EPropertyFilter::All))
	{
		AddError("Failed to load into struct.");
		return false;
	}

	bool bResult = true;
	bResult &= TestEqual(TEXT("Count"), Struct.Map.Num(), 1);

	FEditorConfigTestKey Value = Struct.Map[TEXT("foo")];
	bResult &= TestEqual(TEXT("Name"), Value.Name, TEXT("one"));
	bResult &= TestEqual(TEXT("Number"), Value.Number, 42.0);
	
	ChildConfig.SetRootStruct(Struct, FEditorConfig::EPropertyFilter::All);

	FString Result;
	bResult &= ChildConfig.SaveToString(Result);

	if (!FEditorConfigTestHelpers::AreJsonStringsEquivalent(Result, ChildContents))
	{
		AddError(FString::Printf(TEXT("Contents does not match. Expected:\n%s\n\nActual:\n%s"), *ChildContents, *Result));
		return false;
	}

	return bResult;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditorConfigTests_Load_Object, "EditorConfig.Load.Object", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FEditorConfigTests_Load_Object::RunTest(const FString& Parameters)
{
	const FString Contents = 
R"_JSON({
	"$type": "EditorConfigTestObject",
	"Object": "StaticMesh'/Engine/BasicShapes/Cube.Cube'",
	"Struct": 
	{
		"$type": "EditorConfigTestSimpleStruct",
		"Bool": false,
		"Int": 42,
		"String": "foo",
		"Float": 21.0,
		"Array": [ "foo", "bar" ]
	},
	"Number": 42
})_JSON";


	FEditorConfig Config;
	if (!Config.LoadFromString(Contents))
	{
		AddError("Failed to load from string.");
		return false;
	}

	UEditorConfigTestObject* Object = NewObject<UEditorConfigTestObject>();
	if (!Config.TryGetRootUObject(Object, FEditorConfig::EPropertyFilter::All))
	{
		AddError("Failed to load into object.");
		return false;
	}

	bool bResult = true;
	bResult &= TestTrue(TEXT("Object"), Object->Object != nullptr);
	bResult &= TestEqual(TEXT("Bool"), Object->Struct.Bool, false);
	bResult &= TestEqual(TEXT("Int"), Object->Struct.Int, 42);
	bResult &= TestEqual(TEXT("String"), Object->Struct.String, TEXT("foo"));
	bResult &= TestEqual(TEXT("Float"), Object->Struct.Float, 21.0f);
	bResult &= TestEqual(TEXT("Array"), Object->Struct.Array, TArray<FString> { "foo", "bar"});
	bResult &= TestEqual(TEXT("Number"), Object->Number, 42);
	return bResult;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditorConfigTests_Save_SimpleStruct_Empty, "EditorConfig.Save.SimpleStruct.Empty", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FEditorConfigTests_Save_SimpleStruct_Empty::RunTest(const FString& Parameters)
{
	FEditorConfigTestSimpleStruct Struct;

	FEditorConfig Config;
	Config.SetRootStruct(Struct, FEditorConfig::EPropertyFilter::All);
	
	FString Result;
	if (!Config.SaveToString(Result))
	{
		AddError("Failed to save config.");
		return false;
	}

	const FString Expected =
R"_JSON({ 
	"$type": "EditorConfigTestSimpleStruct"
})_JSON";

	if (!FEditorConfigTestHelpers::AreJsonStringsEquivalent(Result, Expected))
	{
		AddError(FString::Printf(TEXT("Contents does not match. Expected:\n%s\n\nActual:\n%s"), *Expected, *Result));
		return false;
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditorConfigTests_Save_SimpleStruct_Full, "EditorConfig.Save.SimpleStruct.Full", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FEditorConfigTests_Save_SimpleStruct_Full::RunTest(const FString& Parameters)
{
	FEditorConfigTestSimpleStruct Struct;
	Struct.Bool = false;
	Struct.Int = 42;
	Struct.String = "foo";
	Struct.Float = 21.0f;
	Struct.Array = TArray<FString>{ TEXT("foo"), TEXT("bar") };

	FEditorConfig Config;
	Config.SetRootStruct(Struct, FEditorConfig::EPropertyFilter::All);

	FString Result;
	if (!Config.SaveToString(Result))
	{
		AddError("Failed to save config.");
		return false;
	}

	const FString Expected =
R"_JSON({ 
	"$type": "EditorConfigTestSimpleStruct",
	"Bool": false,
	"Int": 42,
	"String": "foo",
	"Float": 21.0,
	"Array": 
	{
		"+": [
			"foo",
			"bar"
		]
	}
})_JSON";

	if (!FEditorConfigTestHelpers::AreJsonStringsEquivalent(Result, Expected))
	{
		AddError(FString::Printf(TEXT("Contents does not match. Expected:\n%s\n\nActual:\n%s"), *Expected, *Result));
		return false;
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditorConfigTests_Save_Object_Full, "EditorConfig.Save.Object.Full", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FEditorConfigTests_Save_Object_Full::RunTest(const FString& Parameters)
{
	UEditorConfigTestObject* Object = NewObject<UEditorConfigTestObject>();
	Object->AddToRoot();

	UObject* Cube = StaticLoadObject(UStaticMesh::StaticClass(), nullptr, TEXT("StaticMesh'/Engine/BasicShapes/Cube.Cube'"));
	if (Cube == nullptr)
	{
		AddError("Unable to find /Engine/BasicShapes/Cube.Cube");
		return false;
	}

	Object->Object = Cube;
	Object->Number = 42;
	Object->Struct.Array = TArray<FString>{ TEXT("foo"), TEXT("bar") };
	Object->Struct.Bool = false;
	Object->Struct.String = TEXT("foo");
	Object->Struct.Int = 42;
	Object->Struct.Float = 21;


	FEditorConfig Config;
	Config.SetRootUObject(Object, FEditorConfig::EPropertyFilter::All);

	FString Result;
	if (!Config.SaveToString(Result))
	{
		AddError("Failed to save config.");
		return false;
	}

	const FString Expected =
R"_JSON({
	"$type": "EditorConfigTestObject",
	"Object": "/Script/Engine.StaticMesh'/Engine/BasicShapes/Cube.Cube'",
	"Struct": 
	{
		"$type": "EditorConfigTestSimpleStruct",
		"Bool": false,
		"Int": 42,
		"String": "foo",
		"Float": 21.0,
		"Array": [ "foo", "bar" ]
	},
	"Number": 42
})_JSON";

	if (!FEditorConfigTestHelpers::AreJsonStringsEquivalent(Result, Expected))
	{
		AddError(FString::Printf(TEXT("Contents does not match. Expected:\n%s\n\nActual:\n%s"), *Expected, *Result));
		return false;
	}

	Object->RemoveFromRoot();

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS
