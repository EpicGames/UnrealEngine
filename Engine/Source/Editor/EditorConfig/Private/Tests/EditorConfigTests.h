// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Serialization/Archive.h"
#include "Templates/TypeHash.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "EditorConfigTests.generated.h"

UENUM()
enum class EEditorConfigTestEnum : uint8
{
	Zero,
	One,
	Two
};

USTRUCT()
struct FEditorConfigTestEnumStruct
{
	GENERATED_BODY()

	UPROPERTY()
	int32 Before = 0;

	UPROPERTY()
	EEditorConfigTestEnum Enum = EEditorConfigTestEnum::Zero;

	UPROPERTY()
	int32 After = 0;
};

USTRUCT()
struct FEditorConfigTestSimpleStruct
{
	GENERATED_BODY()

	UPROPERTY()
	bool Bool = true;

	UPROPERTY()
	int32 Int = 5;

	UPROPERTY(meta=(EditorConfig))
	FString String;

	UPROPERTY()
	float Float = 5.0f;

	UPROPERTY(meta=(EditorConfig))
	TArray<FString> Array;

	bool operator==(const FEditorConfigTestSimpleStruct& Other) const;
	bool operator!=(const FEditorConfigTestSimpleStruct& Other) const;
};

USTRUCT()
struct FEditorConfigTestKey
{
	GENERATED_BODY()
	
	UPROPERTY()
	FString Name;
	
	UPROPERTY()
	double Number = 5.0;

	bool operator==(const FEditorConfigTestKey& Other) const;
	bool operator!=(const FEditorConfigTestKey& Other) const;
};

FORCEINLINE uint32 GetTypeHash(const FEditorConfigTestKey& Struct)
{
	return HashCombine(GetTypeHash(Struct.Name), GetTypeHash(Struct.Number));
}

USTRUCT()
struct FEditorConfigTestComplexArray
{
	GENERATED_BODY()
	
	UPROPERTY()
	TArray<FEditorConfigTestKey> Array;
};

USTRUCT()
struct FEditorConfigTestSimpleMap
{
	GENERATED_BODY()

	UPROPERTY()
	TMap<FString, FString> Map;
};

USTRUCT()
struct FEditorConfigTestSimpleKeyComplexValueMap
{
	GENERATED_BODY()

	UPROPERTY()
	TMap<FString, FEditorConfigTestKey> Map;
};

USTRUCT()
struct FEditorConfigTestComplexMap
{
	GENERATED_BODY()
	
	UPROPERTY()
	TMap<FEditorConfigTestKey, FEditorConfigTestKey> Map;
};

USTRUCT()
struct FEditorConfigTestSimpleSet
{
	GENERATED_BODY()
	
	UPROPERTY()
	TSet<FName> Set;
};

USTRUCT()
struct FEditorConfigTestComplexSet
{
	GENERATED_BODY()
	
	UPROPERTY()
	TSet<FEditorConfigTestKey> Set;
};

UCLASS()
class UEditorConfigTestObject : public UObject
{
	GENERATED_BODY()

public:

	UPROPERTY()
	TObjectPtr<UObject> Object;

	UPROPERTY(meta=(EditorConfig))
	FEditorConfigTestSimpleStruct Struct;

	UPROPERTY(meta=(EditorConfig))
	int32 Number = 5;
};
