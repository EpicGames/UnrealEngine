// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS 

#include "EditorConfigTestHelpers.h"
#include "JsonConfig.h"
#include "Misc/AutomationTest.h"

using namespace UE;

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FJsonConfigTests_LoadBool, "JsonConfig.Load.Bool", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FJsonConfigTests_LoadBool::RunTest(const FString& Parameters)
{
	const FString Contents = 
R"_JSON({ 
	"Bool": true
}
)_JSON";

	FJsonConfig Config;
	if (!Config.LoadFromString(Contents))
	{
		return false;
	}

	bool Bool = false;
	if (!Config.TryGetBool(TEXT("Bool"), Bool))
	{
		return false;
	}

	TestEqual(TEXT("Bool"), Bool, true);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FJsonConfigTests_LoadString, "JsonConfig.Load.String", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FJsonConfigTests_LoadString::RunTest(const FString& Parameters)
{
	const FString Contents = 
R"_JSON({ 
	"Foo": "Foo",
	"Bar": "Bar"
}
)_JSON";

	FJsonConfig Config;
	if (!Config.LoadFromString(Contents))
	{
		return false;
	}

	FString Foo, Bar;
	if (!Config.TryGetString(TEXT("Foo"), Foo) ||
		!Config.TryGetString(TEXT("Bar"), Bar))
	{
		return false;
	}

	TestEqual(TEXT("Foo"), Foo, TEXT("Foo"));
	TestEqual(TEXT("Bar"), Bar, TEXT("Bar"));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FJsonConfigTests_LoadNumber, "JsonConfig.Load.Number", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FJsonConfigTests_LoadNumber::RunTest(const FString& Parameters)
{
	const FString Contents = 
R"_JSON({ 
	"Positive": 63,
	"Negative": -63,
	"PositiveFloat": 12.5,
	"NegativeFloat": -12.5
}
)_JSON";

	FJsonConfig Config;
	if (!Config.LoadFromString(Contents))
	{
		return false;
	}

	uint8 U8;
	uint16 U16;
	uint32 U32;
	uint64 U64;
	int8 I8;
	int16 I16;
	int32 I32;
	int64 I64;
	float Float = 0;
	double Double = 0;

	if (!Config.TryGetNumber(TEXT("Positive"), U8) ||
		!Config.TryGetNumber(TEXT("Positive"), U16) ||
		!Config.TryGetNumber(TEXT("Positive"), U32) ||
		!Config.TryGetNumber(TEXT("Positive"), U64) ||
		!Config.TryGetNumber(TEXT("Positive"), I8) ||
		!Config.TryGetNumber(TEXT("Positive"), I16) ||
		!Config.TryGetNumber(TEXT("Positive"), I32) ||
		!Config.TryGetNumber(TEXT("Positive"), I64) ||
		!Config.TryGetNumber(TEXT("Positive"), Float) ||
		!Config.TryGetNumber(TEXT("Positive"), Double))
	{
		return false;
	}

	TestEqual<uint8>(TEXT("U8"), U8, 63);
	TestEqual<uint16>(TEXT("U16"), U16, 63);
	TestEqual<uint32>(TEXT("U32"), U32, 63);
	TestEqual<uint64>(TEXT("U64"), U64, 63);
	TestEqual<int8>(TEXT("I8"), I8, 63);
	TestEqual<int16>(TEXT("I16"), I16, 63);
	TestEqual<int32>(TEXT("I32"), I32, 63);
	TestEqual<int64>(TEXT("I64"), I64, 63);
	TestEqual(TEXT("Float"), Float, 63.f);
	TestEqual(TEXT("Double"), Double, 63.0);

	if (!Config.TryGetNumber(TEXT("Negative"), I8) ||
		!Config.TryGetNumber(TEXT("Negative"), I16) ||
		!Config.TryGetNumber(TEXT("Negative"), I32) ||
		!Config.TryGetNumber(TEXT("Negative"), I64) ||
		!Config.TryGetNumber(TEXT("Negative"), Float) ||
		!Config.TryGetNumber(TEXT("Negative"), Double))
	{
		return false;
	}

	TestEqual<int8>(TEXT("I8"), I8, -63);
	TestEqual<int16>(TEXT("I16"), I16, -63);
	TestEqual<int32>(TEXT("I32"), I32, -63);
	TestEqual<int64>(TEXT("I64"), I64, -63);
	TestEqual(TEXT("Float"), Float, -63.f);
	TestEqual(TEXT("Double"), Double, -63.0);

	if (!Config.TryGetNumber(TEXT("PositiveFloat"), Float) ||
		!Config.TryGetNumber(TEXT("PositiveFloat"), Double))
	{
		return false;
	}

	TestEqual(TEXT("Float"), Float, 12.5f);
	TestEqual(TEXT("Double"), Double, 12.5);

	if (!Config.TryGetNumber(TEXT("NegativeFloat"), Float) ||
		!Config.TryGetNumber(TEXT("NegativeFloat"), Double))
	{
		return false;
	}

	TestEqual(TEXT("Float"), Float, -12.5f);
	TestEqual(TEXT("Double"), Double, -12.5);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FJsonConfigTests_LoadArrayBool, "JsonConfig.Load.Array.Bool", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FJsonConfigTests_LoadArrayBool::RunTest(const FString& Parameters)
{
	const FString Contents = 
R"_JSON({ 
	"Array": [true, false, true]
}
)_JSON";

	FJsonConfig Config;
	if (!Config.LoadFromString(Contents))
	{
		return false;
	}

	TArray<bool> Array;
	if (!Config.TryGetArray(TEXT("Array"), Array))
	{
		return false;
	}

	TestEqual(TEXT("Size"), Array.Num(), 3);
	TestEqual(TEXT("Contents"), Array, TArray<bool> { true, false, true });

	bool At1;
	if (!Config.TryGetBool(TEXT("Array[1]"), At1))
	{
		return false;
	} 

	TestFalse(TEXT("Indexed"), At1);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FJsonConfigTests_LoadArraySetBool, "JsonConfig.Load.Array.Set.Bool", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FJsonConfigTests_LoadArraySetBool::RunTest(const FString& Parameters)
{
	const FString Contents = 
R"_JSON({ 
	"Array": 
	{
		"=": [true, false, true]
	}
}
)_JSON";

	FJsonConfig Config;
	if (!Config.LoadFromString(Contents))
	{
		return false;
	}

	TArray<bool> Array;
	if (!Config.TryGetArray(TEXT("Array"), Array))
	{
		return false;
	}

	TestEqual(TEXT("Size"), Array.Num(), 3);
	TestEqual(TEXT("Contents"), Array, TArray<bool> { true, false, true });

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FJsonConfigTests_LoadArrayAddBool, "JsonConfig.Load.Array.Add.Bool", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FJsonConfigTests_LoadArrayAddBool::RunTest(const FString& Parameters)
{
	const FString Contents = 
R"_JSON({ 
	"Array": 
	{
		"+": [true, false, true]
	}
}
)_JSON";

	FJsonConfig Config;
	if (!Config.LoadFromString(Contents))
	{
		return false;
	}

	TArray<bool> Array;
	if (!Config.TryGetArray(TEXT("Array"), Array))
	{
		return false;
	}

	TestEqual(TEXT("Size"), Array.Num(), 3);
	TestEqual(TEXT("Contents"), Array, TArray<bool> { true, false, true });
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FJsonConfigTests_LoadNestedObjects, "JsonConfig.Load.Objects.Nested", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FJsonConfigTests_LoadNestedObjects::RunTest(const FString& Parameters)
{
	const FString Contents = 
R"_JSON({ 
	"Top": 
	{
		"Nested":
		{
			"Property": "Foo"
		}
	}
}
)_JSON";

	FJsonConfig Config;
	if (!Config.LoadFromString(Contents))
	{
		return false;
	}

	TSharedPtr<FJsonObject> Top;
	if (!Config.TryGetJsonObject(TEXT("Top"), Top))
	{
		return false;
	}

	TestEqual("Top", Top->Values.Num(), 1);

	TSharedPtr<FJsonObject> Nested;
	if (!Config.TryGetJsonObject(TEXT("Top.Nested"), Nested))
	{
		return false;
	}

	TestEqual(TEXT("Nested"), Top->Values[FJsonObject::FStringType(TEXT("Nested"))]->AsObject(), Nested);

	FString Property;
	if (!Config.TryGetString(TEXT("Top.Nested.Property"), Property))
	{
		return false;
	}

	TestEqual(TEXT("Property"), Property, TEXT("Foo"));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FJsonConfigTests_LoadMapNumericKey, "JsonConfig.Load.Map.NumericKey", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FJsonConfigTests_LoadMapNumericKey::RunTest(const FString& Parameters)
{
	const FString Contents = 
R"_JSON({ 
	"Map":
	[
		{
			"$key": 5,
			"$value": "Foo"
		},
		{
			"$key": 10,
			"$value": "Bar"
		}
	]
}
)_JSON";

	FJsonConfig Config;
	if (!Config.LoadFromString(Contents))
	{
		return false;
	}

	TArray<FJsonValuePair> Map;
	if (!Config.TryGetMap(TEXT("Map"), Map))
	{
		return false;
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FJsonConfigTests_LoadHierarchicalBool, "JsonConfig.Load.Hierarchical.Bool", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FJsonConfigTests_LoadHierarchicalBool::RunTest(const FString& Parameters)
{
	const FString ParentContents = 
R"_JSON({ 
	"Bool": true
}
)_JSON";

	TSharedRef<FJsonConfig> ParentConfig = MakeShared<FJsonConfig>();
	if (!ParentConfig->LoadFromString(ParentContents))
	{
		return false;
	}

	bool Value;
	if (!ParentConfig->TryGetBool(TEXT("Bool"), Value))
	{
		return false;
	}

	TestEqual("Parent Value", Value, true);

	const FString ChildContents = 
R"_JSON({ 
	"Bool": false
}
)_JSON";

	FJsonConfig ChildConfig;
	ChildConfig.SetParent(ParentConfig);

	if (!ChildConfig.TryGetBool(TEXT("Bool"), Value))
	{
		return false;
	}

	TestEqual("Inherited Value", Value, true);

	if (!ChildConfig.LoadFromString(ChildContents))
	{
		return false;
	}

	if (!ChildConfig.TryGetBool(TEXT("Bool"), Value))
	{
		return false;
	}

	TestEqual("Child Value", Value, false);

	// reparent and check
	ChildConfig.SetParent(ParentConfig);

	if (!ChildConfig.TryGetBool(TEXT("Bool"), Value))
	{
		return false;
	}

	TestEqual("Reparented Value", Value, false);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FJsonConfigTests_LoadHierarchicalString, "JsonConfig.Load.Hierarchical.String", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FJsonConfigTests_LoadHierarchicalString::RunTest(const FString& Parameters)
{
	const FString ParentContents = 
R"_JSON({ 
	"String": "Foo"
}
)_JSON";

	TSharedRef<FJsonConfig> ParentConfig = MakeShared<FJsonConfig>();
	if (!ParentConfig->LoadFromString(ParentContents))
	{
		return false;
	}

	FString String;
	if (!ParentConfig->TryGetString(TEXT("String"), String))
	{
		return false;
	}

	TestEqual("Parent Value", String, TEXT("Foo"));

	const FString ChildContents = 
R"_JSON({ 
	"String": "Bar"
}
)_JSON";

	FJsonConfig ChildConfig;
	ChildConfig.SetParent(ParentConfig);

	if (!ChildConfig.TryGetString(TEXT("String"), String))
	{
		return false;
	}

	TestEqual("Inherited Value", String, TEXT("Foo"));

	if (!ChildConfig.LoadFromString(ChildContents))
	{
		return false;
	}

	if (!ChildConfig.TryGetString(TEXT("String"), String))
	{
		return false;
	}

	TestEqual("Child Value", String, TEXT("Bar"));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FJsonConfigTests_LoadHierarchicalObject, "JsonConfig.Load.Hierarchical.Object", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FJsonConfigTests_LoadHierarchicalObject::RunTest(const FString& Parameters)
{
	const FString ParentContents = 
R"_JSON({ 
	"Object":
	{
		"Number": 1,
		"OldString": "Foo"
	}
}
)_JSON";

	TSharedRef<FJsonConfig> ParentConfig = MakeShared<FJsonConfig>();
	if (!ParentConfig->LoadFromString(ParentContents))
	{
		return false;
	}

	int32 Int;
	if (!ParentConfig->TryGetNumber(TEXT("Object.Number"), Int))
	{
		return false;
	}

	TestEqual("Parent Number", Int, 1);

	FString OldString;
	if (!ParentConfig->TryGetString(TEXT("Object.OldString"), OldString))
	{
		return false;
	}

	// check for non-existence of the new string
	FString NewString;
	if (ParentConfig->TryGetString(TEXT("Object.NewString"), NewString))
	{
		return false;
	}

	TestEqual("Parent Old String", OldString, TEXT("Foo"));

	const FString ChildContents = 
R"_JSON({ 
	"Object":
	{
		"Number": 5,
		"NewString": "Bar"
	}
}
)_JSON";

	FJsonConfig ChildConfig;
	ChildConfig.SetParent(ParentConfig);

	if (!ChildConfig.TryGetNumber(TEXT("Object.Number"), Int))
	{
		return false;
	}

	TestEqual("Inherited Int", Int, 1);

	if (!ChildConfig.LoadFromString(ChildContents))
	{
		return false;
	}

	if (!ChildConfig.TryGetNumber(TEXT("Object.Number"), Int))
	{
		return false;
	}

	TestEqual("Child Int", Int, 5);

	if (!ChildConfig.TryGetString(TEXT("Object.OldString"), OldString))
	{
		return false;
	}

	TestEqual("Child Old String", OldString, TEXT("Foo"));

	if (!ChildConfig.TryGetString(TEXT("Object.NewString"), NewString))
	{
		return false;
	}

	TestEqual("Child New String", NewString, TEXT("Bar"));

	return true;
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FJsonConfigTests_LoadHierarchicalMapStringKeyRemove, "JsonConfig.Load.Hierarchical.Map.StringKey.Remove", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FJsonConfigTests_LoadHierarchicalMapStringKeyRemove::RunTest(const FString& Parameters)
{
	const FString ParentContents = 
R"_JSON({ 
	"Map":
	[
		{
			"$key": "Foo",
			"$value": true
		},
		{
			"$key": "Bar",
			"$value": true
		}
	]
}
)_JSON";

	TSharedRef<FJsonConfig> ParentConfig = MakeShared<FJsonConfig>();
	if (!ParentConfig->LoadFromString(ParentContents))
	{
		return false;
	}

	TArray<FJsonValuePair> Map;
	if (!ParentConfig->TryGetMap(TEXT("Map"), Map))
	{
		return false;
	}

	TestEqual(TEXT("Parent Map Size"), Map.Num(), 2);
	
	FString Key = Map[0].Key->AsString();
	TestEqual(TEXT("Parent Map Key"), Key, TEXT("Foo"));

	bool Value = Map[0].Value->AsBool();
	TestEqual(TEXT("Parent Map Value"), Value, true);

	Key = Map[1].Key->AsString();
	TestEqual(TEXT("Parent Map Key"), Key, TEXT("Bar"));

	Value = Map[1].Value->AsBool();
	TestEqual(TEXT("Parent Map Value"), Value, true);

	const FString ChildContents = 
R"_JSON({ 
	"Map":
	{
		"-":
		[
			"Foo"
		]
	}
}
)_JSON";

	FJsonConfig ChildConfig;
	if (!ChildConfig.LoadFromString(ChildContents))
	{
		return false;
	}

	if (!ChildConfig.TryGetMap(TEXT("Map"), Map))
	{
		return false;
	}

	TestEqual(TEXT("Child Map Size"), Map.Num(), 0);

	ChildConfig.SetParent(ParentConfig);

	if (!ChildConfig.TryGetMap(TEXT("Map"), Map))
	{
		return false;
	}

	TestEqual(TEXT("Inherited Map Size"), Map.Num(), 1);

	TestEqual(TEXT("Inherited Map Key"), Map[0].Key->AsString(), TEXT("Bar"));
	TestEqual(TEXT("Inherited Map Value"), Map[0].Value->AsBool(), true);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FJsonConfigTests_LoadHierarchicalMapNumericKeyAdd, "JsonConfig.Load.Hierarchical.Map.NumericKey.Add", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FJsonConfigTests_LoadHierarchicalMapNumericKeyAdd::RunTest(const FString& Parameters)
{
	const FString ParentContents = 
R"_JSON({ 
	"Map":
	[
		{
			"$key": 5,
			"$value": "Foo"
		},
		{
			"$key": 20,
			"$value": "Removed"
		}
	]
}
)_JSON";

	TSharedRef<FJsonConfig> ParentConfig = MakeShared<FJsonConfig>();
	if (!ParentConfig->LoadFromString(ParentContents))
	{
		return false;
	}

	TArray<FJsonValuePair> Map;
	if (!ParentConfig->TryGetMap(TEXT("Map"), Map))
	{
		return false;
	}

	TestEqual("Parent Map Size", Map.Num(), 2);
	
	double Key = Map[0].Key->AsNumber();
	TestEqual("Parent Map Key", Key, 5.0);

	FString Value = Map[0].Value->AsString();
	TestEqual("Parent Map Value", Value, TEXT("Foo"));

	Key = Map[1].Key->AsNumber();
	TestEqual("Parent Map Key", Key, 20.0);

	Value = Map[1].Value->AsString();
	TestEqual("Parent Map Value", Value, TEXT("Removed"));

	const FString ChildContents = 
R"_JSON({ 
	"Map":
	{
		"+":
		[
			{
				"$key": 10,
				"$value": "Bar"
			}
		],
		"-":
		[
			{
				"$key": 20
			}
		]
	}
}
)_JSON";

	FJsonConfig ChildConfig;
	if (!ChildConfig.LoadFromString(ChildContents))
	{
		return false;
	}

	if (!ChildConfig.TryGetMap(TEXT("Map"), Map))
	{
		return false;
	}

	TestEqual("Child Map Size", Map.Num(), 1);

	ChildConfig.SetParent(ParentConfig);

	if (!ChildConfig.TryGetMap(TEXT("Map"), Map))
	{
		return false;
	}

	TestEqual("Inherited Map Size", Map.Num(), 2);

	for (const FJsonValuePair& Pair : Map)
	{
		Key = Pair.Key->AsNumber();
		FString Expected;
		if (Key == 5.0)
		{
			Expected = TEXT("Foo");
		}
		else if (Key == 10.0)
		{
			Expected = TEXT("Bar");
		}
		else
		{
			checkf(false, TEXT("Invalid key found in JSON map."));
		}

		Value = Pair.Value->AsString();
		TestEqual("Inherited Map Value", Value, Expected);
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FJsonConfigTests_LoadHierarchicalMapObjectKeyAdd, "JsonConfig.Load.Hierarchical.Map.ObjectKey.Add", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FJsonConfigTests_LoadHierarchicalMapObjectKeyAdd::RunTest(const FString& Parameters)
{
	const FString ParentContents = 
R"_JSON({ 
	"Map":
	[
		{
			"$key": 
			{
				"Key": "Foo"
			},
			"$value": "Foo"
		}
	]
}
)_JSON";

	TSharedRef<FJsonConfig> ParentConfig = MakeShared<FJsonConfig>();
	if (!ParentConfig->LoadFromString(ParentContents))
	{
		return false;
	}

	TArray<FJsonValuePair> Map;
	if (!ParentConfig->TryGetMap(TEXT("Map"), Map))
	{
		return false;
	}

	TestEqual("Parent Map Size", Map.Num(), 1);

	TSharedPtr<FJsonObject> KeyObject = Map[0].Key->AsObject();
	FString KeyString = KeyObject->GetStringField(TEXT("Key"));
	TestEqual("Parent Map Key", KeyString, TEXT("Foo"));

	TestEqual("Parent Map Value", Map[0].Value->AsString(), TEXT("Foo"));

	const FString ChildContents = 
R"_JSON({ 
	"Map":
	{
		"+":
		[
			{
				"$key": 
				{
					"Key": "Bar"
				},
				"$value": "Bar"
			}
		]
	}
}
)_JSON";

	FJsonConfig ChildConfig;
	if (!ChildConfig.LoadFromString(ChildContents))
	{
		return false;
	}

	if (!ChildConfig.TryGetMap(TEXT("Map"), Map))
	{
		return false;
	}

	TestEqual("Child Map Size", Map.Num(), 1);

	ChildConfig.SetParent(ParentConfig);

	if (!ChildConfig.TryGetMap(TEXT("Map"), Map))
	{
		return false;
	}

	TestEqual("Inherited Map Size", Map.Num(), 2);

	KeyObject = Map[0].Key->AsObject();
	KeyString = KeyObject->GetStringField(TEXT("Key"));
	TestEqual("Inherited Map Key", KeyString, TEXT("Foo"));
	TestEqual("Inherited Map Value", Map[0].Value->AsString(), TEXT("Foo"));

	KeyObject = Map[1].Key->AsObject();
	KeyString = KeyObject->GetStringField(TEXT("Key"));
	TestEqual("Inherited Map Key", KeyString, TEXT("Bar"));
	TestEqual("Inherited Map Value", Map[1].Value->AsString(), TEXT("Bar"));

	// try to override a key with an Add
	const FString OverrideContents = 
R"_JSON({ 
	"Map":
	{
		"+":
		[
			{
				"$key": 
				{
					"Key": "Foo"
				},
				"$value": "Overridden"
			}
		]
	}
}
)_JSON";

	if (!ChildConfig.LoadFromString(OverrideContents))
	{
		return false;
	}

	if (!ChildConfig.TryGetMap(TEXT("Map"), Map))
	{
		return false;
	}

	TestEqual("Child Map Size", Map.Num(), 1);

	ChildConfig.SetParent(ParentConfig);

	if (!ChildConfig.TryGetMap(TEXT("Map"), Map))
	{
		return false;
	}

	TestEqual("Inherited Map Size", Map.Num(), 1);

	KeyObject = Map[0].Key->AsObject();
	KeyString = KeyObject->GetStringField(TEXT("Key"));
	TestEqual("Inherited Map Key", KeyString, TEXT("Foo"));
	TestEqual("Inherited Map Value", Map[0].Value->AsString(), TEXT("Overridden"));

	return true;
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FJsonConfigTests_LoadHierarchicalMapObjectKeyRemove, "JsonConfig.Load.Hierarchical.Map.ObjectKey.Remove", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FJsonConfigTests_LoadHierarchicalMapObjectKeyRemove::RunTest(const FString& Parameters)
{
	const FString ParentContents = 
R"_JSON({ 
	"Map":
	[
		{
			"$key": 
			{
				"Key": "Foo"
			},
			"$value": "Foo"
		},
		{
			"$key": 
			{
				"Key": "Bar"
			},
			"$value": "Bar"
		}
	]
}
)_JSON";

	TSharedRef<FJsonConfig> ParentConfig = MakeShared<FJsonConfig>();
	if (!ParentConfig->LoadFromString(ParentContents))
	{
		return false;
	}

	TArray<FJsonValuePair> Map;
	if (!ParentConfig->TryGetMap(TEXT("Map"), Map))
	{
		return false;
	}

	TestEqual("Parent Map Size", Map.Num(), 2);

	const FString ChildContents = 
R"_JSON({ 
	"Map":
	{
		"-":
		[
			{
				"$key": 
				{
					"Key": "Bar"
				}
			}
		]
	}
}
)_JSON";

	TSharedRef<FJsonConfig> ChildConfig = MakeShared<FJsonConfig>();
	if (!ChildConfig->LoadFromString(ChildContents))
	{
		return false;
	}

	ChildConfig->SetParent(ParentConfig);
	
	if (!ChildConfig->TryGetMap(TEXT("Map"), Map))
	{
		return false;
	}

	TestEqual("Child Map Size", Map.Num(), 1);

	TSharedPtr<FJsonObject> KeyObject = Map[0].Key->AsObject();
	FString KeyString = KeyObject->GetStringField(TEXT("Key"));
	TestEqual("Child Map Key", KeyString, TEXT("Foo"));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FJsonConfigTests_SetNumber, "JsonConfig.Set.Number.Simple", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

template <typename T>
void SetAndCheckNumber(FJsonConfig& Config, T Value)
{
	Config.SetNumber(TEXT("Foo"), Value);

	T Actual;
	ensureAlways(Config.TryGetNumber(TEXT("Foo"), Actual));
	ensureAlways(Value == Actual);
}

bool FJsonConfigTests_SetNumber::RunTest(const FString& Parameters)
{
	FJsonConfig Config;

	// these should all compile and work
	SetAndCheckNumber(Config, (uint8) 1);
	SetAndCheckNumber(Config, (uint16) 2);
	SetAndCheckNumber(Config, (uint32) 3);
	SetAndCheckNumber(Config, (uint64) 4);
	SetAndCheckNumber(Config, (int8) 5);
	SetAndCheckNumber(Config, (int16) 6);
	SetAndCheckNumber(Config, (int32) 7);
	SetAndCheckNumber(Config, (int64) 8);
	SetAndCheckNumber(Config, (float) 9);
	SetAndCheckNumber(Config, (double) 10);

	// this shouldn't compile
	//Config.SetNumber(TEXT("Foo"), TEXT("Bar"));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FJsonConfigTests_SetNumberInArray, "JsonConfig.Set.Number.InArray", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FJsonConfigTests_SetNumberInArray::RunTest(const FString& Parameters)
{
	const FString ParentContents =
R"_JSON({
	"Array": [ 1, 2, 3 ]
})_JSON";

	TSharedRef<FJsonConfig> ParentConfig = MakeShared<FJsonConfig>();
	if (!ParentConfig->LoadFromString(ParentContents))
	{
		return false;
	}

	FJsonConfig ChildConfig;
	ChildConfig.SetParent(ParentConfig);

	TArray<int32> Array;
	if (!ChildConfig.TryGetArray(TEXT("Array"), Array))
	{
		return false;
	}

	TestEqual(TEXT("Before Set"), Array[0], 1);
	TestEqual(TEXT("Before Set"), Array[1], 2);
	TestEqual(TEXT("Before Set"), Array[2], 3);

	ChildConfig.SetNumber(TEXT("Array[0]"), 4);

	if (!ChildConfig.TryGetArray(TEXT("Array"), Array))
	{
		return false;
	}

	// this is somewhat misleading, since loading the override on top of the existing could reorder this
	TestEqual(TEXT("After Set"), Array[0], 4);
	TestEqual(TEXT("After Set"), Array[1], 2);
	TestEqual(TEXT("After Set"), Array[2], 3);

	FString OverrideString;
	ChildConfig.SaveToString(OverrideString);

	const FString ExpectedOverrideString1 = 
R"_JSON({
	"Array":
	{
		"+": [ 4 ],
		"-": [ 1 ]
	}
})_JSON";

	if (!TestTrue(TEXT("Serialized"), FEditorConfigTestHelpers::AreJsonStringsEquivalent(OverrideString, ExpectedOverrideString1)))
	{
		return false;
	}

	// set a different value

	ChildConfig.SetNumber(TEXT("Array[1]"), 5);

	if (!ChildConfig.TryGetArray(TEXT("Array"), Array))
	{
		return false;
	}

	TestEqual(TEXT("After Set"), Array[0], 4);
	TestEqual(TEXT("After Set"), Array[1], 5);
	TestEqual(TEXT("After Set"), Array[2], 3);

	ChildConfig.SaveToString(OverrideString);

	const FString ExpectedOverrideString2 = 
R"_JSON({
	"Array":
	{
		"+": [ 4, 5 ],
		"-": [ 1, 2 ]
	}
})_JSON";

	if (!TestTrue(TEXT("Serialized"), FEditorConfigTestHelpers::AreJsonStringsEquivalent(OverrideString, ExpectedOverrideString2)))
	{
		return false;
	}

	ChildConfig.SetNumber(TEXT("Array[0]"), 1);

	if (!ChildConfig.TryGetArray(TEXT("Array"), Array))
	{
		return false;
	}

	TestEqual(TEXT("After Set"), Array[0], 1);
	TestEqual(TEXT("After Set"), Array[1], 5);
	TestEqual(TEXT("After Set"), Array[2], 3);

	ChildConfig.SaveToString(OverrideString);

	const FString ExpectedOverrideString3 = 
R"_JSON({
	"Array":
	{
		"+": [ 5 ],
		"-": [ 2 ]
	}
})_JSON";

	if (!TestTrue(TEXT("Serialized"), FEditorConfigTestHelpers::AreJsonStringsEquivalent(OverrideString, ExpectedOverrideString3)))
	{
		return false;
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FJsonConfigTests_SetObjectSimple, "JsonConfig.Set.Object.Simple", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FJsonConfigTests_SetObjectSimple::RunTest(const FString& Parameters)
{
	const FString ParentContents =
R"_JSON({
	"Object":
	{
		"Foo": 1,
		"Bar": false
	}
})_JSON";

	TSharedRef<FJsonConfig> ParentConfig = MakeShared<FJsonConfig>();
	if (!ParentConfig->LoadFromString(ParentContents))
	{
		return false;
	}

	FJsonConfig ChildConfig;
	ChildConfig.SetParent(ParentConfig);

	TSharedPtr<FJsonObject> Object;
	if (!ChildConfig.TryGetJsonObject(TEXT("Object"), Object))
	{
		return false;
	}

	TSharedPtr<FJsonValue> FooValue, BarValue;

	FooValue = Object->Values[FJsonObject::FStringType(TEXT("Foo"))];
	TestEqual(TEXT("Before Set"), FooValue->AsNumber(), 1.0);

	BarValue = Object->Values[FJsonObject::FStringType(TEXT("Bar"))];
	TestEqual(TEXT("Before Set"), BarValue->AsBool(), false);

	TSharedPtr<FJsonObject> NewObject = MakeShared<FJsonObject>();
	NewObject->SetNumberField(TEXT("Foo"), 2);

	ChildConfig.SetJsonObject(TEXT("Object"), NewObject);

	if (!ChildConfig.TryGetJsonObject(TEXT("Object"), Object))
	{
		return false;
	}

	FooValue = Object->Values[FJsonObject::FStringType(TEXT("Foo"))];
	TestEqual(TEXT("After Set"), FooValue->AsNumber(), 2.0);

	BarValue = Object->Values[FJsonObject::FStringType(TEXT("Bar"))];
	TestEqual(TEXT("After Set"), BarValue->AsBool(), false);

	const FString ExpectedOverrideString1 = 
R"_JSON({
	"Object":
	{
		"Foo": 2.0
	}
})_JSON";

	FString OverrideString;
	ChildConfig.SaveToString(OverrideString);
	TestTrue(TEXT("Serialized"), FEditorConfigTestHelpers::AreJsonStringsEquivalent(OverrideString, ExpectedOverrideString1));

	NewObject->SetNumberField(TEXT("Foo"), 1.0);
	NewObject->SetBoolField(TEXT("Bar"), true);

	ChildConfig.SetJsonObject(TEXT("Object"), NewObject);

	FooValue = Object->Values[FJsonObject::FStringType(TEXT("Foo"))];
	TestEqual(TEXT("After Set"), FooValue->AsNumber(), 1.0);

	BarValue = Object->Values[FJsonObject::FStringType(TEXT("Bar"))];
	TestEqual(TEXT("After Set"), BarValue->AsBool(), true);

	const FString ExpectedOverrideString2 = 
R"_JSON({
	"Object":
	{
		"Bar": true
	}
})_JSON";

	ChildConfig.SaveToString(OverrideString);
	TestTrue(TEXT("Serialized"), FEditorConfigTestHelpers::AreJsonStringsEquivalent(OverrideString, ExpectedOverrideString2));

	return true;
}

#endif