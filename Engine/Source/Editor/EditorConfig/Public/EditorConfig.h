// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/EqualityComparable.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Delegates/Delegate.h"
#include "Dom/JsonObject.h"
#include "HAL/Platform.h"
#include "JsonConfig.h"
#include "Misc/AssertionMacros.h"
#include "Templates/IsPointer.h"
#include "Templates/Models.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/UnrealTypeTraits.h"

#define UE_API EDITORCONFIG_API

class FArrayProperty;
class FJsonObject;
class FJsonValue;
class FMapProperty;
class FProperty;
class FSetProperty;
class FString;
class UClass;
class UObject;
class UStruct;

class FEditorConfig
{
public:
	enum class EPropertyFilter
	{
		All,
		MetadataOnly
	};

	UE_API FEditorConfig();

	UE_API void SetParent(TSharedPtr<FEditorConfig> InConfig);

	UE_API bool LoadFromString(FStringView Content);
	UE_API bool LoadFromFile(FStringView FilePath);
	UE_API bool SaveToString(FString& OutResult) const;

	bool IsValid() const { return JsonConfig.IsValid() && JsonConfig->IsValid(); }

	// UStruct & UObject
	template <typename T>
	bool TryGetStruct(FStringView Key, T& OutValue, EPropertyFilter Filter = EPropertyFilter::MetadataOnly) const;
	template <typename T>
	bool TryGetUObject(FStringView Key, T& OutValue, EPropertyFilter Filter = EPropertyFilter::MetadataOnly) const;

	template <typename T>
	bool TryGetRootStruct(T& OutValue, EPropertyFilter Filter = EPropertyFilter::MetadataOnly) const;
	template <typename T>
	bool TryGetRootUObject(T& OutValue, EPropertyFilter Filter = EPropertyFilter::MetadataOnly) const;
	template <typename T>
	bool TryGetRootUObject(T* OutValue, EPropertyFilter Filter = EPropertyFilter::MetadataOnly) const;

	UE_API bool TryGetRootStruct(const UStruct* Class, void* OutValue, EPropertyFilter Filter = EPropertyFilter::MetadataOnly) const;
	UE_API bool TryGetRootUObject(const UClass* Class, UObject* OutValue, EPropertyFilter Filter = EPropertyFilter::MetadataOnly) const;

	template <typename T>
	void SetStruct(FStringView Key, const T& InValue, EPropertyFilter Filter = EPropertyFilter::MetadataOnly);
	template <typename T>
	void SetUObject(FStringView Key, const T& InValue, EPropertyFilter Filter = EPropertyFilter::MetadataOnly);

	template <typename T>
	void SetRootStruct(const T& InValue, EPropertyFilter Filter = EPropertyFilter::MetadataOnly);
	template <typename T>
	void SetRootUObject(const T& InValue, EPropertyFilter Filter = EPropertyFilter::MetadataOnly);

	UE_API void SetRootStruct(const UStruct* Class, const void* Instance, EPropertyFilter Filter = EPropertyFilter::MetadataOnly);
	UE_API void SetRootUObject(const UClass* Class, const UObject* Instance, EPropertyFilter Filter = EPropertyFilter::MetadataOnly);

	UE_API bool HasOverride(FStringView Key) const;

	UE_API void OnSaved();

	DECLARE_EVENT_OneParam(FEditorConfig, FOnEditorConfigDirtied, const FEditorConfig&);
	FOnEditorConfigDirtied& OnEditorConfigDirtied() { return EditorConfigDirtiedEvent; }

private:

	friend class UEditorConfigSubsystem; // for access to LoadFromFile and SaveToFile

	static UE_API void ReadUObject(const TSharedPtr<FJsonObject>& JsonObject, const UClass* Class, UObject* Instance, EPropertyFilter Filter);
	static UE_API void ReadStruct(const TSharedPtr<FJsonObject>& JsonObject, const UStruct* Struct, void* Instance, UObject* Owner, EPropertyFilter Filter);
	static UE_API void ReadValue(const TSharedPtr<FJsonValue>& JsonValue, const FProperty* Property, void* DataPtr, UObject* Owner);
	
	static UE_API TSharedPtr<FJsonObject> WriteStruct(const UStruct* Struct, const void* Instance, const void* Defaults, EPropertyFilter Filter);
	static UE_API TSharedPtr<FJsonObject> WriteUObject(const UClass* Class, const UObject* Instance, EPropertyFilter Filter);
	static UE_API TSharedPtr<FJsonValue> WriteArray(const FArrayProperty* ArrayProperty, const void* DataPtr);
	static UE_API TSharedPtr<FJsonValue> WriteSet(const FSetProperty* Property, const void* DataPtr);
	static UE_API TSharedPtr<FJsonValue> WriteMap(const FMapProperty* Property, const void* DataPtr);
	static UE_API TSharedPtr<FJsonValue> WriteValue(const FProperty* Property, const void* DataPtr, const void* Defaults);

	UE_API bool SaveToFile(FStringView FilePath) const;

	UE_API void SetDirty();

private:
	TSharedPtr<UE::FJsonConfig> JsonConfig;
	TSharedPtr<FEditorConfig> ParentConfig;
		
	FOnEditorConfigDirtied EditorConfigDirtiedEvent;
	bool Dirty { false };
};

template <typename T>
bool FEditorConfig::TryGetStruct(FStringView Key, T& OutValue, EPropertyFilter Filter) const
{
	if (!IsValid())
	{
		return false;
	}

	TSharedPtr<FJsonObject> StructData;
	UE::FJsonPath Path(Key);
	if (!JsonConfig->TryGetJsonObject(Path, StructData))
	{
		return false;
	}

	if (!StructData.IsValid())
	{
		return false;
	}

	const UStruct* Struct = T::StaticStruct();
	ReadStruct(StructData, Struct, &OutValue, nullptr, Filter);

	return true;
}

template <typename T>
bool FEditorConfig::TryGetUObject(FStringView Key, T& OutValue, EPropertyFilter Filter) const
{
	static_assert(TIsDerivedFrom<T, UObject>::Value, "Type is not derived from UObject.");

	if (!IsValid())
	{
		return false;
	}

	TSharedPtr<FJsonObject> UObjectData;
	UE::FJsonPath Path(Key);
	if (!JsonConfig->TryGetJsonObject(Path, UObjectData))
	{
		return false;
	}

	if (!UObjectData.IsValid())
	{
		return false;
	}

	const UClass* Class = T::StaticClass();
	ReadUObject(UObjectData, Class, &OutValue, Filter);

	return true;
}

template <typename T>
bool FEditorConfig::TryGetRootStruct(T& OutValue, EPropertyFilter Filter) const
{
	return TryGetRootStruct(T::StaticStruct(), &OutValue, Filter);
}

template <typename T>
bool FEditorConfig::TryGetRootUObject(T& OutValue, EPropertyFilter Filter) const
{
	static_assert(TIsDerivedFrom<T, UObject>::Value, "Type is not derived from UObject.");

	return TryGetRootUObject(T::StaticClass(), &OutValue, Filter);
}

template <typename T>
bool FEditorConfig::TryGetRootUObject(T* OutValue, EPropertyFilter Filter) const
{
	static_assert(TIsDerivedFrom<T, UObject>::Value, "Type is not derived from UObject.");
	checkf(OutValue != nullptr, TEXT("Output value was null."));

	return TryGetRootUObject(T::StaticClass(), OutValue, Filter);
}

template <typename T>
void FEditorConfig::SetStruct(FStringView Key, const T& InValue, EPropertyFilter Filter)
{
	if (!IsValid())
	{
		return;
	}

	TSharedPtr<FJsonObject> JsonObject = WriteStruct(T::StaticStruct(), &InValue, nullptr, Filter);
	JsonConfig->SetJsonObject(UE::FJsonPath(Key), JsonObject);

	SetDirty();
}

template <typename T>
void FEditorConfig::SetUObject(FStringView Key, const T& InValue, EPropertyFilter Filter)
{
	static_assert(TIsDerivedFrom<T, UObject>::Value, "Type is not derived from UObject.");

	if (!IsValid())
	{
		return;
	}
	
	TSharedPtr<FJsonObject> JsonObject = WriteUObject(T::StaticClass(), &InValue);
	JsonConfig->SetJsonObject(UE::FJsonPath(Key), JsonObject);
		
	SetDirty();
}

template <typename T>
void FEditorConfig::SetRootStruct(const T& InValue, EPropertyFilter Filter)
{
	SetRootStruct(T::StaticStruct(), &InValue, Filter);
}

template <typename T>
void FEditorConfig::SetRootUObject(const T& InValue, EPropertyFilter Filter)
{
	if constexpr (TIsPointer<T>::Value)
	{
		static_assert(TIsDerivedFrom<typename TRemovePointer<T>::Type, UObject>::Value, "Type is not derived from UObject.");
		checkf(InValue != nullptr, TEXT("Object value was null."));

		SetRootUObject(TRemovePointer<T>::Type::StaticClass(), InValue, Filter);
	}
	else
	{
		static_assert(TIsDerivedFrom<typename TRemovePointer<T>::Type, UObject>::Value, "Type is not derived from UObject.");

		SetRootUObject(T::StaticClass(), &InValue, Filter);
	}
}

#undef UE_API
