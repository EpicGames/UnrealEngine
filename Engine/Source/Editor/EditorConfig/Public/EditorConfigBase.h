// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "EditorConfigBase.generated.h"

#define UE_API EDITORCONFIG_API

/** Inherit from this class to simplify saving and loading properties from editor configs. */
UCLASS(MinimalAPI)
class UEditorConfigBase : public UObject
{
	GENERATED_BODY()

public:

	/** Load any properties of this class into properties marked with metadata tag "EditorConfig" from the class's EditorConfig */
	UE_API bool LoadEditorConfig();

	/** Save any properties of this class in properties marked with metadata tag "EditorConfig" into the class's EditorConfig. */
	UE_API bool SaveEditorConfig() const;
};

#undef UE_API
