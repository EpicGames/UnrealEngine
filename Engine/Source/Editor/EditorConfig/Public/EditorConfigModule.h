// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Logging/LogMacros.h"

DECLARE_LOG_CATEGORY_EXTERN(LogEditorConfig, Log, All);

class FEditorConfigModule : public IModuleInterface
{
};
