// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Editor.h"
#include "EditorConfig.h"
#include "EditorSubsystem.h"
#include "TickableEditorObject.h"

#include "EditorConfigSubsystem.generated.h"

#define UE_API EDITORCONFIG_API

UCLASS(MinimalAPI)
class UEditorConfigSubsystem : 
	public UEditorSubsystem,
	public FTickableEditorObject
{
	GENERATED_BODY()

public:
	UE_API UEditorConfigSubsystem();

	/** 
	 * Load a config from the root of the JSON file into a given UObject.
	 * This loads the config from the UCLASS's EditorConfig="ConfigName" metadata.
	 * @param Object The object to load into.
	 * @param Filter Whether to load all properties, or only ones marked with the EditorConfig metadata.
	 */
	template <typename TObject>
	bool LoadConfigObject(TObject* Object, FEditorConfig::EPropertyFilter Filter = FEditorConfig::EPropertyFilter::MetadataOnly);

	/** 
	 * Save the given UObject to the root of the JSON config.
	 * This saves the config to the UCLASS's EditorConfig="ConfigName" metadata.
	 * @param Object The object to save.
	 * @param Filter Whether to save all properties, or only ones marked with the EditorConfig metadata.
	 */
	template <typename TObject>
	bool SaveConfigObject(const TObject* Object, FEditorConfig::EPropertyFilter Filter = FEditorConfig::EPropertyFilter::MetadataOnly);

	/** 
	 * Load a config from the root of the JSON file into a given UObject.
	 * This loads the config from the UCLASS's EditorConfig="ConfigName" metadata.
	 * @param Class The UClass of the object. 
	 * @param Object The UObject instance to load into.
	 * @param Filter Whether to load all properties, or only ones marked with the EditorConfig metadata.
	 */
	UE_API bool LoadConfigObject(const UClass* Class, UObject* Object, FEditorConfig::EPropertyFilter = FEditorConfig::EPropertyFilter::MetadataOnly);
	
	/** 
	 * Save the given UObject of the given class to the root of the JSON config.
	 * This saves the config to the UCLASS's EditorConfig="ConfigName" metadata.
	 * @param Class The UClass of the object. 
	 * @param Object The UObject instance to save.
	 * @param Filter Whether to save all properties, or only ones marked with the EditorConfig metadata.
	 */
	UE_API bool SaveConfigObject(const UClass* Class, const UObject* Object, FEditorConfig::EPropertyFilter = FEditorConfig::EPropertyFilter::MetadataOnly);

	enum class ESearchDirectoryType : uint8
	{
		Engine,
		Project,
		ProjectOverrides,
		User
	};
	
	/** 
	 * Find a config with the given name that has already been loaded, load it if it hasn't been, or create one with the given name.
	 * 
	 * @param IncludedTypes The "lowest" level of search directory types to include, ex. If IncludedTypes is set to Project, only Engine and Project paths will be loaded
	 */
	UE_API TSharedRef<FEditorConfig> FindOrLoadConfig(FStringView ConfigName, ESearchDirectoryType IncludedTypes = ESearchDirectoryType::User);

	/*
	 * Save the given config to the location it was loaded.
	 */
	UE_API void SaveConfig(TSharedRef<FEditorConfig> Config);

	/** Force reload the given config and all its (current and potential) parents from disk. */
	UE_API bool ReloadConfig(TSharedRef<FEditorConfig> Config);

	/**
	 * Append a new config search directory to the given type.
	 * Engine directories are searched first, then Project, then ProjectOverrides, then User.
	 */
	UE_API void AddSearchDirectory(ESearchDirectoryType Type, FStringView SearchDir);

	/**
	 * Append a new config search directory to the given type early.
	 * Engine directories are searched first, then Project, then ProjectOverrides, then User.
	 * 
	 * Note: this function can only be called before the UEditorConfigSubsystem is initialized otherwise it will assert. 
	 * This function is useful if there is a need to register a layer of config before any config read could happen.
	 */
	static UE_API void EarlyAddSearchDirectory(ESearchDirectoryType Type, FStringView SearchDir);

private:
	UE_API void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API void Deinitialize() override;
	UE_API void OnSaveCompleted(TSharedPtr<FEditorConfig> Config);
	UE_API void OnEditorConfigDirtied(const FEditorConfig& Config);

	/** FTickableEditorObject interface */
	UE_API void Tick(float DeltaTime) override;
	UE_API TStatId GetStatId() const override;

private:
	struct FPendingSave
	{
		FString FileName;
		TSharedPtr<FEditorConfig> Config;
		TFuture<bool> WasSuccess;
		float TimeSinceQueued { 0 };
	};

	FCriticalSection SaveLock;
	TArray<FPendingSave> PendingSaves;
	TArray<TPair<ESearchDirectoryType, FString>> SearchDirectories;
	TMap<FString, TSharedPtr<FEditorConfig>> LoadedConfigs;

	static UE_API TArray<TPair<ESearchDirectoryType, FString>> EarlyRegistredSearchDirectories;
};

template <typename TObject>
bool UEditorConfigSubsystem::LoadConfigObject(TObject* Object, FEditorConfig::EPropertyFilter Filter)
{
	static_assert(TIsDerivedFrom<TObject, UObject>::Value, "Type is not derived from UObject.");
	check(Object != nullptr);
	
	const UClass* Class = TObject::StaticClass();
	return LoadConfigObject(Class, Object, Filter);
}

template <typename TObject>
bool UEditorConfigSubsystem::SaveConfigObject(const TObject* Object, FEditorConfig::EPropertyFilter Filter)
{
	static_assert(TIsDerivedFrom<TObject, UObject>::Value, "Type is not derived from UObject.");
	check(Object != nullptr);

	const UClass* Class = TObject::StaticClass();
	return SaveConfigObject(Class, Object);
}

#undef UE_API
