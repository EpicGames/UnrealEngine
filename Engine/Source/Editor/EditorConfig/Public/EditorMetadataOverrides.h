// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/SparseArray.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EditorConfig.h"
#include "EditorSubsystem.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "EditorMetadataOverrides.generated.h"

#define UE_API EDITORCONFIG_API

class FEditorConfig;
class FField;
class FSubsystemCollectionBase;
class UClass;
class UObject;
class UStruct;

USTRUCT()
struct FMetadataSet
{
	GENERATED_BODY()

	// map of metadata key to metadata value
	UPROPERTY()
	TMap<FName, FString> Strings;
		
	UPROPERTY()
	TMap<FName, bool> Bools;

	UPROPERTY()
	TMap<FName, int32> Ints;
		
	UPROPERTY()
	TMap<FName, float> Floats;
};

USTRUCT()
struct FStructMetadata
{
	GENERATED_BODY()

	// map of field name to field metadata
	UPROPERTY()
	TMap<FName, FMetadataSet> Fields;

	UPROPERTY()
	FMetadataSet StructMetadata;
};

USTRUCT()
struct FMetadataConfig
{
	GENERATED_BODY()
		
	// map of class name to class metadata
	UPROPERTY()
	TMap<FName, FStructMetadata> Classes;
};

UENUM()
enum class EMetadataType
{
	None,
	Bool,
	Int,
	Float,
	String
};

UCLASS(MinimalAPI)
class UEditorMetadataOverrides : 
	public UEditorSubsystem
{ 
	GENERATED_BODY()

public:
	UE_API UEditorMetadataOverrides();
	virtual ~UEditorMetadataOverrides() = default;

	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;

	UE_API bool LoadFromConfig(TSharedPtr<FEditorConfig> Config);
	UE_API void Save();

	UE_API EMetadataType GetMetadataType(const FField* Field, FName Key) const;
	UE_API EMetadataType GetMetadataType(const UStruct* Struct, FName Key) const;

	UE_API bool GetStringMetadata(const FField* Field, FName Key, FString& OutValue) const;
	UE_API void SetStringMetadata(const FField* Field, FName Key, FStringView Value);

	UE_API bool GetFloatMetadata(const FField* Field, FName Key, float& OutValue) const;
	UE_API void SetFloatMetadata(const FField* Field, FName Key, float Value);

	UE_API bool GetIntMetadata(const FField* Field, FName Key, int32& OutValue) const;
	UE_API void SetIntMetadata(const FField* Field, FName Key, int32 Value);

	UE_API bool GetBoolMetadata(const FField* Field, FName Key, bool& OutValue) const;
	UE_API void SetBoolMetadata(const FField* Field, FName Key, bool Value);

	UE_API bool GetClassMetadata(const FField* Field, FName Key, UClass*& OutValue) const;
	UE_API void SetClassMetadata(const FField* Field, FName Key, UClass* Value);

	UE_API bool GetArrayMetadata(const FField* Field, FName Key, TArray<FString>& OutValue) const;
	UE_API void SetArrayMetadata(const FField* Field, FName Key, const TArray<FString>& Value);

	UE_API void AddToArrayMetadata(const FField* Field, FName Key, const FString& Value);
	UE_API void RemoveFromArrayMetadata(const FField* Field, FName Key, const FString& Value);

	UE_API void RemoveMetadata(const FField* Field, FName Key);

	UE_API bool GetStringMetadata(const UStruct* Struct, FName Key, FString& OutValue) const;
	UE_API void SetStringMetadata(const UStruct* Struct, FName Key, FStringView Value);

	UE_API bool GetFloatMetadata(const UStruct* Struct, FName Key, float& OutValue) const;
	UE_API void SetFloatMetadata(const UStruct* Struct, FName Key, float Value);

	UE_API bool GetIntMetadata(const UStruct* Struct, FName Key, int32& OutValue) const;
	UE_API void SetIntMetadata(const UStruct* Struct, FName Key, int32 Value);

	UE_API bool GetBoolMetadata(const UStruct* Struct, FName Key, bool& OutValue) const;
	UE_API void SetBoolMetadata(const UStruct* Struct, FName Key, bool Value);

	UE_API bool GetClassMetadata(const UStruct* Struct, FName Key, UClass*& OutValue) const;
	UE_API void SetClassMetadata(const UStruct* Struct, FName Key, UClass* Value);

	UE_API bool GetArrayMetadata(const UStruct* Struct, FName Key, TArray<FString>& OutValue) const;
	UE_API void SetArrayMetadata(const UStruct* Struct, FName Key, const TArray<FString>& Value);

	UE_API void AddToArrayMetadata(const UStruct* Struct, FName Key, const FString& Value);
	UE_API void RemoveFromArrayMetadata(const UStruct* Struct, FName Key, const FString& Value);

	UE_API void RemoveMetadata(const UStruct* Struct, FName Key);

private:
	UE_API const FMetadataSet* FindFieldMetadata(const FField* Field) const;
	UE_API FMetadataSet* FindFieldMetadata(const FField* Field);
	UE_API FMetadataSet* FindOrAddFieldMetadata(const FField* Field);

	UE_API const FMetadataSet* FindStructMetadata(const UStruct* Struct) const;
	UE_API FMetadataSet* FindStructMetadata(const UStruct* Struct);
	UE_API FMetadataSet* FindOrAddStructMetadata(const UStruct* Struct);

	UE_API void OnCompleted(bool bSuccess);

private:
	TSharedPtr<FEditorConfig> SourceConfig;
	FMetadataConfig LoadedMetadata;
};

#undef UE_API
