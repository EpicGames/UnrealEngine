// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Async.h"
#include "Async/Future.h"
#include "Concepts/EqualityComparable.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Dom/JsonObject.h"
#include "Templates/SharedPointer.h"

class FJsonObject;
class FJsonValue;
class FName;
class FText;

namespace UE
{
	class FJsonPath
	{
	public:
		struct FPart
		{
			FString Name;
			int32 Index = INDEX_NONE;
		};

		EDITORCONFIG_API FJsonPath();
		EDITORCONFIG_API FJsonPath(const TCHAR* Path);
		EDITORCONFIG_API FJsonPath(FStringView Path);
		EDITORCONFIG_API FJsonPath(const FJsonPath& Other);
		EDITORCONFIG_API FJsonPath(FJsonPath&& Other);

		bool IsValid() const { return Length() > 0; }
		const int32 Length() const { return PathParts.Num(); }

		EDITORCONFIG_API void Append(FStringView Name);
		EDITORCONFIG_API void SetArrayIndex(int32 Index);

		EDITORCONFIG_API FJsonPath GetSubPath(int32 NumParts) const;

		EDITORCONFIG_API FString ToString() const;

		const FPart& operator[](int32 Idx) const { return PathParts[Idx]; }
		const TArray<FPart>& GetAll() const { return PathParts; }

	private:
		EDITORCONFIG_API void ParsePath(const FString& InPath);

	private:
		TArray<FPart> PathParts;
	};

	using FJsonValuePair = TPair<TSharedPtr<FJsonValue>, TSharedPtr<FJsonValue>>;

	class FJsonConfig
	{
	public:
		EDITORCONFIG_API FJsonConfig();

		EDITORCONFIG_API void SetParent(const TSharedPtr<FJsonConfig>& Parent);

		EDITORCONFIG_API bool LoadFromFile(FStringView FilePath);
		EDITORCONFIG_API bool LoadFromString(FStringView Content);
	
		EDITORCONFIG_API bool SaveToFile(FStringView FilePath) const;
		EDITORCONFIG_API bool SaveToString(FString& OutResult) const;

		bool IsValid() const { return MergedObject.IsValid(); }

		EDITORCONFIG_API const FJsonConfig* GetParentConfig() const;

		template <typename T>
		bool TryGetNumber(const FJsonPath& Path, T& OutValue) const;
		EDITORCONFIG_API bool TryGetBool(const FJsonPath& Path, bool& OutValue) const;
		EDITORCONFIG_API bool TryGetString(const FJsonPath& Path, FString& OutValue) const;
		EDITORCONFIG_API bool TryGetString(const FJsonPath& Path, FName& OutValue) const;
		EDITORCONFIG_API bool TryGetString(const FJsonPath& Path, FText& OutValue) const;
		EDITORCONFIG_API bool TryGetJsonValue(const FJsonPath& Path, TSharedPtr<FJsonValue>& OutValue) const;
		EDITORCONFIG_API bool TryGetJsonObject(const FJsonPath& Path, TSharedPtr<FJsonObject>& OutValue) const;
		EDITORCONFIG_API bool TryGetArray(const FJsonPath& Path, TArray<TSharedPtr<FJsonValue>>& OutArray) const;

		EDITORCONFIG_API TSharedPtr<FJsonObject> GetRootObject() const;

		// these are specializations for arithmetic and string arrays
		// these could be templated with enable-ifs, 
		// but it ended up being more lines of incomprehensible template SFINAE than this clear list of types is
		EDITORCONFIG_API bool TryGetArray(const FJsonPath& Path, TArray<bool>& OutArray) const;
		EDITORCONFIG_API bool TryGetArray(const FJsonPath& Path, TArray<int8>& OutArray) const;
		EDITORCONFIG_API bool TryGetArray(const FJsonPath& Path, TArray<int16>& OutArray) const;
		EDITORCONFIG_API bool TryGetArray(const FJsonPath& Path, TArray<int32>& OutArray) const;
		EDITORCONFIG_API bool TryGetArray(const FJsonPath& Path, TArray<int64>& OutArray) const;
		EDITORCONFIG_API bool TryGetArray(const FJsonPath& Path, TArray<uint8>& OutArray) const;
		EDITORCONFIG_API bool TryGetArray(const FJsonPath& Path, TArray<uint16>& OutArray) const;
		EDITORCONFIG_API bool TryGetArray(const FJsonPath& Path, TArray<uint32>& OutArray) const;
		EDITORCONFIG_API bool TryGetArray(const FJsonPath& Path, TArray<uint64>& OutArray) const;
		EDITORCONFIG_API bool TryGetArray(const FJsonPath& Path, TArray<float>& OutArray) const;
		EDITORCONFIG_API bool TryGetArray(const FJsonPath& Path, TArray<double>& OutArray) const;
		EDITORCONFIG_API bool TryGetArray(const FJsonPath& Path, TArray<FString>& OutArray) const;
		EDITORCONFIG_API bool TryGetArray(const FJsonPath& Path, TArray<FText>& OutArray) const;
		EDITORCONFIG_API bool TryGetArray(const FJsonPath& Path, TArray<FName>& OutArray) const;

		EDITORCONFIG_API bool TryGetMap(const FJsonPath& Path, TArray<FJsonValuePair>& OutMap) const;

		// try to set a number - returns false if:
		// - the path references an object that doesn't exist
		// - the path references an index in an array that doesn't exist
		template <typename T>
		bool SetNumber(const FJsonPath& Path, T Value);
		EDITORCONFIG_API bool SetBool(const FJsonPath& Path, bool Value);
		EDITORCONFIG_API bool SetString(const FJsonPath& Path, FStringView Value);
		EDITORCONFIG_API bool SetString(const FJsonPath& Path, const FText& Value);
		EDITORCONFIG_API bool SetJsonValue(const FJsonPath& Path, const TSharedPtr<FJsonValue>& Value);
		EDITORCONFIG_API bool SetJsonObject(const FJsonPath& Path, const TSharedPtr<FJsonObject>& Object);
		EDITORCONFIG_API bool SetJsonArray(const FJsonPath& Path, const TArray<TSharedPtr<FJsonValue>>& Array);

		EDITORCONFIG_API bool SetRootObject(const TSharedPtr<FJsonObject>& Object);

		EDITORCONFIG_API bool HasOverride(const FJsonPath& Path) const;

	private:

		template <typename T, typename TGetter>
		bool TryGetArrayHelper(const FJsonPath& Path, TArray<T>& OutArray, TGetter Getter) const;

		template <typename T>
		bool TryGetNumericArrayHelper(const FJsonPath& Path, TArray<T>& OutArray) const;

		EDITORCONFIG_API bool SetJsonValueInMerged(const FJsonPath& Path, const TSharedPtr<FJsonValue>& Value);
		EDITORCONFIG_API bool SetJsonValueInOverride(const FJsonPath& Path, const TSharedPtr<FJsonValue>& NewValue, const TSharedPtr<FJsonValue>& PreviousValue, const TSharedPtr<FJsonValue>& ParentValue);
		EDITORCONFIG_API bool SetArrayValueInOverride(const TSharedPtr<FJsonValue>& CurrentValue, const TArray<TSharedPtr<FJsonValue>>& NewArray, const TSharedPtr<FJsonValue>& ParentValue);
		EDITORCONFIG_API bool SetObjectValueInOverride(const TSharedPtr<FJsonObject>& CurrentObject, const TSharedPtr<FJsonObject>& NewObject, const TSharedPtr<FJsonValue>& ParentValue);

		EDITORCONFIG_API bool RemoveJsonValueFromOverride(const FJsonPath& Path, const TSharedPtr<FJsonValue>& PreviousValue);

		EDITORCONFIG_API bool MergeThisWithParent();
		EDITORCONFIG_API void OnParentConfigChanged();

	private:

		TFuture<void> SaveFuture;

		FSimpleDelegate OnConfigChanged;

		TSharedPtr<FJsonConfig> ParentConfig;

		TSharedPtr<FJsonObject> OverrideObject;
		TSharedPtr<FJsonObject> MergedObject;
	};
}

#include "JsonConfig.inl" // IWYU pragma: export
