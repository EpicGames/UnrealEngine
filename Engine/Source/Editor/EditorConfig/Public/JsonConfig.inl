// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace UE
{
	template <typename T>
	bool FJsonConfig::TryGetNumber(const FJsonPath& Path, T& OutValue) const
	{
		TSharedPtr<FJsonValue> JsonValue;
		if (!TryGetJsonValue(Path, JsonValue))
		{
			return false;
		}

		return JsonValue->TryGetNumber(OutValue);
	}

	template <typename T, typename TGetter>
	bool FJsonConfig::TryGetArrayHelper(const FJsonPath& Path, TArray<T>& OutArray, TGetter Getter) const
	{
		if (!IsValid())
		{
			return false;
		}

		TArray<TSharedPtr<FJsonValue>> JsonValueArray;
		if (!TryGetArray(Path, JsonValueArray))
		{
			return false;
		}

		OutArray.Reset();

		for (const TSharedPtr<FJsonValue>& JsonValue : JsonValueArray)
		{
			T Value;
			if (!Getter(JsonValue, Value))
			{
				OutArray.Reset();
				return false;
			}

			OutArray.Add(Value);
		}

		return true;
	}

	template <typename T>
	bool FJsonConfig::TryGetNumericArrayHelper(const FJsonPath& Path, TArray<T>& OutArray) const
	{
		return TryGetArrayHelper(Path, OutArray, 
			[](const TSharedPtr<FJsonValue>& JsonValue, T& OutNumber)
			{
				return JsonValue->TryGetNumber(OutNumber);
			});
	}

	template <typename T>
	bool FJsonConfig::SetNumber(const FJsonPath& Path, T Value)
	{
		static_assert(TIsArithmetic<T>::Value, "Value type must be a number.");

		TSharedPtr<FJsonValue> JsonValue = MakeShared<FJsonValueNumber>((double) Value);
		return SetJsonValue(Path, JsonValue);
	}
}
